// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "HAL/PlatformFile.h"

class IStorageServerPlatformFile : public IPlatformFile
{
public:

	struct FConnectionStats
	{
		uint64 AccumulatedBytes = 0;
		uint32 RequestCount = 0;
		double MinRequestThroughput = 0.0;
		double MaxRequestThroughput = 0.0;
	};

	virtual FStringView GetHostAddr() const = 0;

	virtual void GetAndResetConnectionStats(FConnectionStats& OutStats) = 0;
	virtual void UpdateFileList() = 0;
};
