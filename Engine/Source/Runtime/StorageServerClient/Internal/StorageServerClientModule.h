// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "HAL/IPlatformFileModule.h"
#include "HAL/PlatformFileManager.h"
#include "IStorageServerPlatformFile.h"
#include "IO/IoDispatcherBackend.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Serialization/PackageStore.h"

#if !UE_BUILD_SHIPPING

class IStorageServerClientModule : public IPlatformFileModule
{
public:
	static FORCEINLINE IStorageServerClientModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IStorageServerClientModule>("StorageServerClient");
	}
	static FORCEINLINE IStorageServerPlatformFile* FindStorageServerPlatformFile()
	{
		return static_cast<IStorageServerPlatformFile*>(FPlatformFileManager::Get().FindPlatformFile(TEXT("StorageServer")));
	}

	virtual IStorageServerPlatformFile* TryCreateCustomPlatformFile(FStringView StoreDirectory, IPlatformFile* Inner) = 0;

	// Creates a platform file, package store and io dispatcher backends,
	// without mounting them in FPackageStore and FIoDispatcher.
	struct FDetachedPlatformFileResult
	{
		TUniquePtr<IStorageServerPlatformFile> PlatformFile;
		TSharedPtr<IPackageStoreBackend> PackageStoreBackend;
		TSharedPtr<IIoDispatcherBackend> IoDispatcherBackend;
	};
	virtual bool CreateDetachedPlatformFile(FString Host, uint16 Port, FString Project, FString Platform, FDetachedPlatformFileResult& OutResult) = 0;
};

#endif // !UE_BUILD_SHIPPING
