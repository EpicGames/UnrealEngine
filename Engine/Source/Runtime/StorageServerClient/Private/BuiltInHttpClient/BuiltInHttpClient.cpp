// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuiltInHttpClient.h"
#include "StorageServerConnection.h"
#include "HAL/PlatformProcess.h"
#include "Math/UnrealMathUtility.h"

#if !UE_BUILD_SHIPPING

static inline FAnsiStringView GetMimeTypeString(EStorageServerContentType ContentType)
{
	switch (ContentType)
	{
		case EStorageServerContentType::CbObject:
			return ANSITEXTVIEW("application/x-ue-cb");
		case EStorageServerContentType::Binary:
			return ANSITEXTVIEW("application/octet-stream");
		case EStorageServerContentType::CompressedBinary:
			return ANSITEXTVIEW("application/x-ue-comp");
		default:
			return ANSITEXTVIEW("unknown");
	};
};

static inline EStorageServerContentType GetMimeType(const FAnsiStringView& ContentType)
{
	if (ContentType == ANSITEXTVIEW("application/octet-stream"))
	{
		return EStorageServerContentType::Binary;
	}
	else if (ContentType == ANSITEXTVIEW("application/x-ue-comp"))
	{
		return EStorageServerContentType::CompressedBinary;
	}
	else if (ContentType == ANSITEXTVIEW("application/x-ue-cb"))
	{
		return EStorageServerContentType::CbObject;
	}
	else
	{
		return EStorageServerContentType::Unknown;
	}
};

FBuiltInHttpClient::FBuiltInHttpClient(TUniquePtr<IBuiltInHttpClientSocketPool> InSocketPool, FString InHostname)
	: SocketPool(MoveTemp(InSocketPool))
	, Hostname(InHostname)
{
}

void FBuiltInHttpClient::SetPersistentHeaders(TArrayView<const FAnsiString> InHeaders)
{
	PersistentHeaders = InHeaders;
}

IStorageServerHttpClient::FResult FBuiltInHttpClient::RequestSync(
	FAnsiStringView Url,
	EStorageServerContentType Accept,
	FAnsiStringView Verb,
	TOptional<FIoBuffer> OptPayload,
	EStorageServerContentType PayloadContentType,
	TOptional<FIoBuffer> OptDestination,
	float TimeoutSeconds,
	const bool bReportErrors
)
{
	const bool bHasPayload = OptPayload.IsSet() && OptPayload->GetSize() > 0;

	TAnsiStringBuilder<1024> HeaderBuffer;

	HeaderBuffer
		<< Verb << " " << Url << " HTTP/1.1\r\n" 
		<< "Host: " << TCHAR_TO_ANSI(*Hostname) << "\r\n"
		<< "Connection: Keep-Alive\r\n";
	if (Accept != EStorageServerContentType::Unknown)
	{
		HeaderBuffer
			<< "Accept: " << GetMimeTypeString(Accept) << "\r\n";
	}
	if (bHasPayload)
	{
		HeaderBuffer
			<< "Content-Length: " << OptPayload->GetSize() << "\r\n";
	}
	if (PayloadContentType != EStorageServerContentType::Unknown)
	{
		HeaderBuffer
			<< "Content-Type: " << GetMimeTypeString(PayloadContentType) << "\r\n";
	}
	for (const FAnsiString& Header : PersistentHeaders)
	{
		HeaderBuffer << Header << "\r\n";
	}
	HeaderBuffer << "\r\n";

	IBuiltInHttpClientSocket* Socket = nullptr;

	bool bSendOk = false;

	const int32 AttemptCount = TimeoutSeconds <= 0.0f ? 10 : 1;
	for (int32 Attempt = 0; Attempt < AttemptCount; Attempt++)
	{
		if (Attempt > 0)
		{
			// Bounded exponential backoff (10ms, 20ms, 40ms, ... capped at 500ms) so we
			// don't hammer a server that briefly refused us due to a full accept backlog.
			const int32 Shift = FMath::Min(Attempt - 1, 6);
			const float BackoffSeconds = FMath::Min(0.5f, 0.01f * (float)(1 << Shift));
			FPlatformProcess::Sleep(BackoffSeconds);
		}

		if ((Socket = SocketPool->AcquireSocket(TimeoutSeconds)) == nullptr)
		{
			continue;
		}

		if (Socket->Send(reinterpret_cast<const uint8*>(HeaderBuffer.GetData()), HeaderBuffer.Len()) &&
			(!bHasPayload || Socket->Send(OptPayload->GetData(), OptPayload->GetSize())))
		{
			bSendOk = true;
			break;
		}

		SocketPool->ReleaseSocket(Socket, false);
	}

	if (!bSendOk)
	{
		if (bReportErrors)
		{
			UE_LOGF(LogStorageServerConnection, Fatal, "Failed sending request to storage server.");
		}
		else
		{
			// Caller (e.g. HandshakeRequest) is responsible for retrying / reporting the
			// outer failure. Log here at Display so a silent retry loop still tells you
			// the request never reached the server - the typical signal that the peer
			// closed or RST'd the socket between connect() and send().
			UE_LOGF(LogStorageServerConnection, Display, "Send to storage server failed (peer likely closed the connection): %.*hs %.*hs", Verb.Len(), Verb.GetData(), Url.Len(), Url.GetData());
		}

		return IStorageServerHttpClient::FResult{
			.IoStatus       = TIoStatusOr<FIoBuffer>(FIoStatus(EIoErrorCode::WriteError)),
			.ContentType    = EStorageServerContentType::Unknown,
			.HttpStatusCode = 0
		};
	}

	bool bRecvOk = true;

	uint8 ReadResponseLineBuffer[1024];
	auto ReadResponseLine = [&ReadResponseLineBuffer, &Socket, &bRecvOk]() -> FAnsiStringView
	{
		for (;;)
		{
			uint64 BytesRead = 0;
			Socket->Recv(ReadResponseLineBuffer, 1024, BytesRead, ESocketReceiveFlags::Peek);
			FAnsiStringView ResponseView(reinterpret_cast<const ANSICHAR*>(ReadResponseLineBuffer), BytesRead);
			int32 LineEndIndex = 0;
			if (ResponseView.FindChar('\r', LineEndIndex) && BytesRead >= LineEndIndex + 2)
			{
				check(ResponseView[LineEndIndex + 1] == '\n');
				if (!Socket->Recv(ReadResponseLineBuffer, LineEndIndex + 2, BytesRead, ESocketReceiveFlags::None))
				{
					bRecvOk = false;
				}
				check(BytesRead == LineEndIndex + 2);
				return ResponseView.Left(LineEndIndex);
			}
		}
	};

	int64 ContentLength = 0;
	int32 StatusCode = 500;
	EStorageServerContentType ContentType = EStorageServerContentType::Unknown;
	while (true)
	{
		FAnsiStringView ResponseLine = ReadResponseLine();

		if (ResponseLine.IsEmpty())
		{
			break;
		}
		else if (ResponseLine.StartsWith("HTTP/1.1 "))
		{
			StatusCode = TCString<ANSICHAR>::Atoi64(ResponseLine.GetData() + 9);
		}
		else if (ResponseLine.StartsWith("Content-Length: "))
		{
			ContentLength = FMath::Max(0, TCString<ANSICHAR>::Atoi64(ResponseLine.GetData() + 16));
		}
		else if (ResponseLine.StartsWith("Content-Type: "))
		{
			ContentType = GetMimeType(ResponseLine.RightChop(14));
		}
	}

	const bool bIsOk = StatusCode < 300; // we don't handle redirects here, so everything >=300 is considered an error
	const EIoErrorCode ErrorCode = bIsOk ? EIoErrorCode::Ok : (StatusCode == 404 ? EIoErrorCode::NotFound : EIoErrorCode::Unknown);

	TOptional<FIoBuffer> ResponsePayload = ContentLength > 0
		? (OptDestination.IsSet() && ((int64)OptDestination->GetSize() >= ContentLength) ? OptDestination.GetValue() : FIoBuffer(ContentLength))
		: FIoBuffer(0);

	if (ContentLength)
	{
		int64 TotalBytesRead = 0;
		while (TotalBytesRead < ContentLength)
		{
			uint64 BytesRead = 0;
			if (!Socket->Recv(ResponsePayload->GetData() + TotalBytesRead, ContentLength - TotalBytesRead, BytesRead, ESocketReceiveFlags::WaitAll))
			{
				bRecvOk = false;
				break;
			}

			TotalBytesRead += (int64)BytesRead;
		}

		ResponsePayload->SetSize(TotalBytesRead);
	}

	const bool bHasResponsePayload = ResponsePayload.IsSet() && ResponsePayload->GetSize() > 0;

	const bool bKeepAlive = bSendOk && bRecvOk;
	SocketPool->ReleaseSocket(Socket, bKeepAlive);

	if (bIsOk)
	{
		return IStorageServerHttpClient::FResult{
			.IoStatus       = TIoStatusOr<FIoBuffer>(MoveTemp(ResponsePayload.GetValue())),
			.ContentType    = ContentType,
			.HttpStatusCode = StatusCode
		};
	}
	else
	{
		FString ErrorMessage = bHasResponsePayload ? FString::ConstructFromPtrSize(reinterpret_cast<const ANSICHAR*>(ResponsePayload->GetData()), ResponsePayload->GetSize()) : TEXT("Unknown error");

		// Log every non-2xx so silent-retry callers (e.g. HandshakeRequest) and callers
		// that surface the error themselves both leave a record of the HTTP status and
		// the server-supplied error body. Pairs with the "Send to storage server failed"
		// log above to disambiguate "server closed the socket" from "server returned
		// HTTP error" at the cost of one extra Display line per failed request.
		UE_LOGF(LogStorageServerConnection, Display, "Storage server returned HTTP %d for %.*hs: %ls", StatusCode, Url.Len(), Url.GetData(), *ErrorMessage);

		return IStorageServerHttpClient::FResult{
			.IoStatus       = TIoStatusOr<FIoBuffer>(FIoStatus(ErrorCode, ErrorMessage)),
			.ContentType    = ContentType,
			.HttpStatusCode = StatusCode
		};
	}
}

#endif