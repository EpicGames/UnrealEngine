// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StorageServerHttpClient.h"
#include "SocketTypes.h"

#if !UE_BUILD_SHIPPING

class IBuiltInHttpClientSocket
{
public:
	virtual ~IBuiltInHttpClientSocket() = default;

	virtual bool Send(const uint8* Data, const uint64 DataSize)                                                      = 0;
	virtual bool Recv(uint8* Data, const uint64 DataSize, uint64& BytesRead, ESocketReceiveFlags::Type ReceiveFlags) = 0;
	virtual bool HasPendingData(uint64& PendingDataSize) const                                                       = 0;
	virtual void Close()                                                                                             = 0;

	// Probe whether a pooled socket is still usable. Returns true only when the
	// connection is open and idle (no buffered data, no peer-closed state). A busy
	// server can reap idle keep-alive connections, so pools must check before reusing.
	virtual bool IsAlive() const = 0;
};

class IBuiltInHttpClientSocketPool
{
public:
	virtual ~IBuiltInHttpClientSocketPool() = default;

	virtual IBuiltInHttpClientSocket* AcquireSocket(float TimeoutSeconds = -1.f) = 0;
	virtual void ReleaseSocket(IBuiltInHttpClientSocket* Socket, bool bKeepAlive) = 0;
};

class FBuiltInHttpClient : public IStorageServerHttpClient
{
public:
	FBuiltInHttpClient(TUniquePtr<IBuiltInHttpClientSocketPool> InSocketPool, FString InHostname);
	virtual ~FBuiltInHttpClient() override = default;

	virtual void SetPersistentHeaders(TArrayView<const FAnsiString> InHeaders) override;

	virtual FResult RequestSync(
		FAnsiStringView Url,
		EStorageServerContentType Accept,
		FAnsiStringView Verb,
		TOptional<FIoBuffer> OptPayload,
		EStorageServerContentType PayloadContentType,
		TOptional<FIoBuffer> OptDestination,
		float TimeoutSeconds,
		const bool bReportErrors
	) override;

	virtual void RequestAsync(
		FResultCallback&& Callback,
		FAnsiStringView Url,
		EStorageServerContentType Accept,
		FAnsiStringView Verb,
		TOptional<FIoBuffer> OptPayload,
		EStorageServerContentType PayloadContentType,
		TOptional<FIoBuffer> OptDestination,
		float TimeoutSeconds,
		const bool bReportErrors
	) override
	{
		// This HTTP client doesn't support async requests
		Callback(RequestSync(Url, Accept, Verb, OptPayload, PayloadContentType, OptDestination, TimeoutSeconds, bReportErrors));
	}
private:
	TUniquePtr<IBuiltInHttpClientSocketPool> SocketPool;
	FString Hostname;
	TArray<FAnsiString> PersistentHeaders;
};

#endif
