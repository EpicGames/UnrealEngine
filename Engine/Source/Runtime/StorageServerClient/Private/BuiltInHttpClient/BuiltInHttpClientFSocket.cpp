// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuiltInHttpClientFSocket.h"
#include "StorageServerConnection.h"
#include "Sockets.h"
#include "SocketSubsystem.h"
#include "Misc/ScopeExit.h"

#if !UE_BUILD_SHIPPING

FBuiltInHttpClientFSocket::FBuiltInHttpClientFSocket(FSocket* InSocket)
	: Socket(InSocket)
{
}

FBuiltInHttpClientFSocket::~FBuiltInHttpClientFSocket()
{
	if (Socket)
	{
		Socket->Close();
		delete Socket;
		Socket = nullptr;
	}
}

bool FBuiltInHttpClientFSocket::Send(const uint8* Data, const uint64 DataSize)
{
	if (!Socket)
	{
		return false;
	}

	uint64 TotalBytesSent = 0;
	while (TotalBytesSent < DataSize)
	{
		const uint64 BytesRemaining = DataSize - TotalBytesSent;
		const int32 BytesToSend = (int32)FMath::Min<uint64>(BytesRemaining, (uint64)INT32_MAX);
		int32 BytesSent = 0;
		if (!Socket->Send(Data + TotalBytesSent, BytesToSend, BytesSent))
		{
			return false;
		}
		if (BytesSent <= 0)
		{
			// No forward progress - abort instead of spinning. A well-behaved FSocket
			// should never report success with zero bytes sent, but guard regardless.
			return false;
		}
		TotalBytesSent += (uint64)BytesSent;
	}

	return true;
}

bool FBuiltInHttpClientFSocket::Recv(uint8* Data, const uint64 DataSize, uint64& BytesRead, ESocketReceiveFlags::Type ReceiveFlags)
{
	if (!Socket)
	{
		return false;
	}

	int32 ReadBytes = 0;
	if (!Socket->Recv(Data, DataSize, ReadBytes, ReceiveFlags))
	{
		return false;
	}

	BytesRead = ReadBytes;
	return true;
}

bool FBuiltInHttpClientFSocket::HasPendingData(uint64& PendingDataSize) const
{
	uint32 PendingData;
	bool bRes = Socket->HasPendingData(PendingData);

	PendingDataSize = PendingData;
	return bRes;
}

void FBuiltInHttpClientFSocket::Close()
{
	Socket->Close();
}

bool FBuiltInHttpClientFSocket::IsAlive() const
{
	if (!Socket)
	{
		return false;
	}

	if (Socket->GetConnectionState() != SCS_Connected)
	{
		return false;
	}

	// A pooled keep-alive socket should have nothing readable. If select()-style Wait
	// reports readability, that means either buffered data (poisoned - previous response
	// not drained) or peer EOF/error - both reasons to discard rather than reuse.
	if (Socket->Wait(ESocketWaitConditions::WaitForRead, FTimespan::Zero()))
	{
		return false;
	}

	return true;
}

FBuiltInHttpClientFSocketPool::FBuiltInHttpClientFSocketPool(TSharedPtr<FInternetAddr> InServerAddr, ISocketSubsystem& InSocketSubsystem)
	: ServerAddr(InServerAddr)
	, SocketSubsystem(InSocketSubsystem)
{
}

FBuiltInHttpClientFSocketPool::~FBuiltInHttpClientFSocketPool()
{
	IBuiltInHttpClientSocket* Socket = nullptr;
	while ((Socket = SocketPool.Pop()) != nullptr)
	{
		delete Socket;
	}
}

IBuiltInHttpClientSocket* FBuiltInHttpClientFSocketPool::AcquireSocket(float TimeoutSeconds)
{
	// Drain stale sockets out of the pool. A busy server commonly reaps idle keep-alive
	// connections, leaving us holding peer-closed sockets that look fine until we try to
	// recv from them; check liveness explicitly before reuse.
	while (IBuiltInHttpClientSocket* Pooled = SocketPool.Pop())
	{
		if (Pooled->IsAlive())
		{
			return Pooled;
		}
		UE_LOGF(LogStorageServerConnection, Verbose, "Discarded stale pooled FSocket");
		delete Pooled;
	}

	if (ServerAddr.IsValid())
	{
		FSocket* Socket = SocketSubsystem.CreateSocket(NAME_Stream, TEXT("StorageServer"), ServerAddr->GetProtocolType());
		
		Socket->SetNoDelay(true);
		
		if (TimeoutSeconds > 0.0f)
		{
			Socket->SetNonBlocking(true);
			
			ON_SCOPE_EXIT
			{
				Socket->SetNonBlocking(false);
			};
			
			if (Socket->Connect(*ServerAddr) && Socket->Wait(ESocketWaitConditions::WaitForWrite, FTimespan::FromSeconds(TimeoutSeconds)))
			{
				return new FBuiltInHttpClientFSocket(Socket);
			}
		}
		else
		{
			if (Socket->Connect(*ServerAddr))
			{
				return new FBuiltInHttpClientFSocket(Socket);
			}
		}
		
		delete Socket;
	}
	return nullptr;
}

void FBuiltInHttpClientFSocketPool::ReleaseSocket(IBuiltInHttpClientSocket* Socket, bool bKeepAlive)
{
	uint64 PendingDataSize = 0;
	if (bKeepAlive && !Socket->HasPendingData(PendingDataSize))
	{
		SocketPool.Push(Socket);
	}
	else
	{
		if (PendingDataSize > 0)
		{
			UE_LOGF(LogStorageServerConnection, Fatal, "Socket was not fully drained");
		}

		delete Socket;
	}
}

#endif
