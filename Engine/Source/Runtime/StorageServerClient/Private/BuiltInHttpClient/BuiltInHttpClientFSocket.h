// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BuiltInHttpClient.h"
#include "Containers/LockFreeList.h"

#if !UE_BUILD_SHIPPING

class FSocket;

class FBuiltInHttpClientFSocket : public IBuiltInHttpClientSocket
{
public:
	FBuiltInHttpClientFSocket(FSocket* InSocket);
	virtual ~FBuiltInHttpClientFSocket() override;

	virtual bool Send(const uint8* Data, const uint64 DataSize) override;
	virtual bool Recv(uint8* Data, const uint64 DataSize, uint64& BytesRead, ESocketReceiveFlags::Type ReceiveFlags) override;
	virtual bool HasPendingData(uint64& PendingDataSize) const override;
	virtual void Close() override;
	virtual bool IsAlive() const override;

private:
	FSocket* Socket;
};

class FBuiltInHttpClientFSocketPool : public IBuiltInHttpClientSocketPool
{
public:
	FBuiltInHttpClientFSocketPool(TSharedPtr<FInternetAddr> InServerAddr, ISocketSubsystem& InSocketSubsystem);
	virtual ~FBuiltInHttpClientFSocketPool() override;

	virtual IBuiltInHttpClientSocket* AcquireSocket(float TimeoutSeconds = -1.f) override;
	virtual void ReleaseSocket(IBuiltInHttpClientSocket* Socket, bool bKeepAlive) override;

private:
	TSharedPtr<FInternetAddr> ServerAddr;
	ISocketSubsystem& SocketSubsystem;
	TLockFreePointerListUnordered<IBuiltInHttpClientSocket, PLATFORM_CACHE_LINE_SIZE> SocketPool;
};

#endif