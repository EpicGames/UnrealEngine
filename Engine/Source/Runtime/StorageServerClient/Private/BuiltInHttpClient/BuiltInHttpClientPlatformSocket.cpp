// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuiltInHttpClientPlatformSocket.h"
#include "HAL/PlatformProcess.h"

#if !UE_BUILD_SHIPPING

DEFINE_LOG_CATEGORY_STATIC(LogStorageServerPlatformBackend, Log, All);

FBuiltInHttpClientPlatformSocket::FBuiltInHttpClientPlatformSocket(IPlatformHostCommunication* InCommunication, IPlatformHostSocketPtr InSocket, int32 InProtocolNumber)
	: Communication(InCommunication)
	, Socket(InSocket)
	, ConnectionBuffer(1024 * 64)
	, ProtocolNumber(InProtocolNumber)
{
}

FBuiltInHttpClientPlatformSocket::~FBuiltInHttpClientPlatformSocket()
{
	Close();
}

bool FBuiltInHttpClientPlatformSocket::Send(const uint8* Data, const uint64 DataSize)
{
	if (!Socket || Socket->GetState() != IPlatformHostSocket::EConnectionState::Connected)
	{
		return false;
	}

	const bool bSuccess = Socket->Send(Data, DataSize) == IPlatformHostSocket::EResultNet::Ok;
	return bSuccess;
}

bool FBuiltInHttpClientPlatformSocket::Recv(uint8* Data, const uint64 DataSize, uint64& BytesRead, ESocketReceiveFlags::Type ReceiveFlags)
{
	if (!Socket || Socket->GetState() != IPlatformHostSocket::EConnectionState::Connected)
	{
		return false;
	}

	if (ConnectionBuffer.GetCapacity() < DataSize)
	{
		UE_LOG(LogStorageServerPlatformBackend, Display, TEXT("ConnectionBuffer capacity is lower than requested data read (%" UINT64_FMT " vs %" UINT64_FMT ")"), ConnectionBuffer.GetCapacity(), DataSize);
	}

	BytesRead = 0;
	if (ConnectionBuffer.IsEmpty())
	{
		uint8 Buffer[1024];
		if (Socket->Receive(Buffer, sizeof(Buffer), BytesRead, IPlatformHostSocket::EReceiveFlags::DontWait) != IPlatformHostSocket::EResultNet::Ok)
		{
			return false;
		}

		if (!ConnectionBuffer.Put(Buffer, BytesRead))
		{
			UE_LOGF(LogStorageServerPlatformBackend, Display, "Couldn't fit the received data in the connection buffer");
		}
	}

	if (ReceiveFlags == ESocketReceiveFlags::Peek)
	{
		ConnectionBuffer.Peek(Data, DataSize, BytesRead);
	}
	else if (ReceiveFlags == ESocketReceiveFlags::WaitAll)
	{
		ConnectionBuffer.Consume(Data, DataSize, BytesRead);
		uint64 TotalBytesRead = BytesRead;

		while (TotalBytesRead < DataSize)
		{
			const uint64 BytesToRead = DataSize - TotalBytesRead;
			if (Socket->Receive(Data + TotalBytesRead, BytesToRead, BytesRead, IPlatformHostSocket::EReceiveFlags::WaitAll) != IPlatformHostSocket::EResultNet::Ok)
			{
				return false;
			}

			TotalBytesRead += BytesRead;

			if (TotalBytesRead > DataSize)
			{
				UE_LOGF(LogStorageServerPlatformBackend, Display, "Exceeded what was supposed to be downloaded");
			}
		}

		BytesRead = TotalBytesRead;
	}
	else if (ReceiveFlags == ESocketReceiveFlags::None)
	{
		ConnectionBuffer.Consume(Data, DataSize, BytesRead);
	}

	return true;
}

bool FBuiltInHttpClientPlatformSocket::HasPendingData(uint64& PendingDataSize) const
{
	PendingDataSize = ConnectionBuffer.GetSize();
	return PendingDataSize > 0;
}

void FBuiltInHttpClientPlatformSocket::Close()
{
	Communication->CloseConnection(Socket);
}

bool FBuiltInHttpClientPlatformSocket::IsAlive() const
{
	if (!Socket || Socket->GetState() != IPlatformHostSocket::EConnectionState::Connected)
	{
		return false;
	}

	// Any buffered bytes indicate the previous response wasn't fully drained; treat as
	// poisoned and drop rather than reusing.
	if (!ConnectionBuffer.IsEmpty())
	{
		return false;
	}

	// The IPlatformHostSocket interface has no peek primitive. Probe peer-EOF / error
	// state by attempting a non-blocking read of a single byte. A healthy idle pooled
	// socket reports Ok with zero bytes; anything else (transport error, disconnected
	// host, or unexpectedly pending bytes) means we must not reuse this socket.
	//
	// If a byte is actually read here it is intentionally discarded along with the
	// socket - returning a poisoned socket to the caller would corrupt the next
	// request's response stream.
	uint8 ProbeByte = 0;
	uint64 BytesReceived = 0;
	const IPlatformHostSocket::EResultNet ProbeResult =
		Socket->Receive(&ProbeByte, sizeof(ProbeByte), BytesReceived, IPlatformHostSocket::EReceiveFlags::DontWait);

	if (ProbeResult != IPlatformHostSocket::EResultNet::Ok)
	{
		return false;
	}

	if (BytesReceived > 0)
	{
		return false;
	}

	return true;
}

FBuiltInHttpClientPlatformSocketPool::FBuiltInHttpClientPlatformSocketPool(const FString InAddress)
	: Address(InAddress)
{
	Communication = &FPlatformMisc::GetPlatformHostCommunication();

	UsedSockets.Init(false, 10); // TODO add host communication to get available amount of socket connection
}

FBuiltInHttpClientPlatformSocketPool::~FBuiltInHttpClientPlatformSocketPool()
{
	IBuiltInHttpClientSocket* Socket = nullptr;
	while ((Socket = SocketPool.Pop()) != nullptr)
	{
		delete Socket;
	}
}

IBuiltInHttpClientSocket* FBuiltInHttpClientPlatformSocketPool::AcquireSocket(float TimeoutSeconds)
{
	// Drain stale sockets out of the pool before reusing - matches the policy used by
	// the other transports (UDS, FSocket).
	while (IBuiltInHttpClientSocket* Pooled = SocketPool.Pop())
	{
		if (Pooled->IsAlive())
		{
			return Pooled;
		}
		UE_LOGF(LogStorageServerPlatformBackend, Verbose, "Discarded stale pooled platform socket");
		delete Pooled;
	}

	int32 ProtocolNumber = -1;
	while (ProtocolNumber == -1)
	{
		FScopeLock Lock(&UsedSocketsCS);
		ProtocolNumber = UsedSockets.Find(false);

		if (ProtocolNumber == -1)
		{
			// All sockets are in use, and we have limited amount of sockets we could use
			UsedSocketsCV.Wait(UsedSocketsCS);
		}
		else
		{
			UsedSockets[ProtocolNumber] = true;
			break;
		}
	}

	IPlatformHostSocket::EConnectionState ConnectionState = IPlatformHostSocket::EConnectionState::Unknown;

	// TODO use address to specify which device to connect to
	IPlatformHostSocketPtr PlatformSocket = Communication->OpenConnection(ProtocolNumber, *FString::Printf(TEXT("PlatformSocket %d"), ProtocolNumber));
	if (PlatformSocket)
	{
		float WaitingFor = 0.f;
		const float SleepTime = 0.01f;
		ConnectionState = PlatformSocket->GetState();
		while (ConnectionState == IPlatformHostSocket::EConnectionState::Created)
		{
			if ((TimeoutSeconds != -1.f) && (WaitingFor > TimeoutSeconds))
			{
				UE_LOGF(LogStorageServerPlatformBackend, Error, "Platform connection timed out");
				break;
			}

			FPlatformProcess::Sleep(SleepTime);
			WaitingFor += SleepTime;
			ConnectionState = PlatformSocket->GetState();
		}
	}

	if (ConnectionState == IPlatformHostSocket::EConnectionState::Connected)
	{
		return new FBuiltInHttpClientPlatformSocket(Communication, PlatformSocket, ProtocolNumber);
	}
	else
	{
		if (PlatformSocket)
		{
			Communication->CloseConnection(PlatformSocket);
		}

		FScopeLock Lock(&UsedSocketsCS);
		UsedSockets[ProtocolNumber] = false;
		return nullptr;
	}
}

void FBuiltInHttpClientPlatformSocketPool::ReleaseSocket(IBuiltInHttpClientSocket* Socket, bool bKeepAlive)
{
	uint64 PendingDataSize = 0;
	if (bKeepAlive && !Socket->HasPendingData(PendingDataSize))
	{
		SocketPool.Push(Socket);
	}
	else
	{
		const int32 ProtocolNumber = static_cast<FBuiltInHttpClientPlatformSocket*>(Socket)->GetProtocolNumber();

		if (PendingDataSize > 0)
		{
			UE_LOGF(LogStorageServerPlatformBackend, Fatal, "Socket was not fully drained");
		}

		delete Socket;

		{
			FScopeLock Lock(&UsedSocketsCS);
			UsedSockets[ProtocolNumber] = false;
			UsedSocketsCV.NotifyOne();
		}
	}
}

#endif