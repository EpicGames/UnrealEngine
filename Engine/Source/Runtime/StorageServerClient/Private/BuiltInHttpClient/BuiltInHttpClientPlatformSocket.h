// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BuiltInHttpClient.h"
#include "Experimental/Async/ConditionVariable.h"
#include "GenericPlatform/GenericPlatformHostCommunication.h"
#include "GenericPlatform/GenericPlatformHostSocket.h"
#include "Containers/LockFreeList.h"

#if !UE_BUILD_SHIPPING

class FConnectionCircularBuffer
{
public:
	explicit FConnectionCircularBuffer(uint64 InCapacity)
		: CapacityMask(FMath::RoundUpToPowerOfTwo(InCapacity) - 1)
	{
		Allocation = new uint8[GetCapacity()];
	}

	~FConnectionCircularBuffer()
	{
		delete[] Allocation;
	}

	uint8 operator[](uint64 Index) const
	{
		return const_cast<FConnectionCircularBuffer*>(this)->operator[](Index);
	}

	uint8& operator[](uint64 Index)
	{
		check(Index < GetSize());
		uint64 DataOffset = (Index + Tail) & (CapacityMask);
		return Allocation[DataOffset];
	}

	uint64 GetSize() const
	{
		return Size;
	}

	uint64 GetCapacity() const
	{
		return CapacityMask + 1;
	}

	bool IsEmpty() const
	{
		return Size == 0;
	}

	void Clear()
	{
		Tail = 0;
		Head = 0;
		Size = 0;
	}

	uint64 SpaceLeft() const
	{
		return GetCapacity() - Size;
	}

	void Peek(uint8* Data, const uint64 DataSize, uint64& OutSize)
	{
		OutSize = FMath::Min(DataSize, Size);

		if (OutSize + Tail > GetCapacity())
		{
			uint64 LenToEnd = GetCapacity() - Tail;
			uint64 LenFromBegin = OutSize - LenToEnd;
			FMemory::Memcpy(Data, Allocation + Tail, LenToEnd);
			FMemory::Memcpy(Data + LenToEnd, Allocation, LenFromBegin);
		}
		else
		{
			FMemory::Memcpy(Data, Allocation + Tail, OutSize);
		}
	}

	void Consume(uint8* Data, const uint64 DataSize, uint64& OutSize)
	{
		Peek(Data, DataSize, OutSize);
		Tail = (Tail + OutSize) & CapacityMask;

		Size -= OutSize;

		if (Size == 0)
		{
			Head = 0;
			Tail = 0;
		}
	}

	bool Put(uint8* Data, uint64 DataSize)
	{
		if (DataSize > SpaceLeft())
		{
			return false;
		}

		if (DataSize + Head > GetCapacity())
		{
			uint64 LenToEnd = GetCapacity() - Head;
			uint64 LenFromBegin = DataSize - LenToEnd;
			FMemory::Memcpy(Allocation + Head, Data, LenToEnd);
			FMemory::Memcpy(Allocation, Data + LenToEnd, LenFromBegin);
		}
		else
		{
			FMemory::Memcpy(Allocation + Head, Data, DataSize);
		}

		Head = (Head + DataSize) & (CapacityMask);

		Size += DataSize;

		return true;
	}

private:
	uint8* Allocation{ nullptr };
	uint64 CapacityMask{ 0 };
	uint64 Size{ 0 };
	uint64 Head{ 0 };
	uint64 Tail{ 0 };
};

class FBuiltInHttpClientPlatformSocket : public IBuiltInHttpClientSocket
{
public:
	FBuiltInHttpClientPlatformSocket(IPlatformHostCommunication* InCommunication, IPlatformHostSocketPtr InSocket, int32 InProtocolNumber);
	virtual ~FBuiltInHttpClientPlatformSocket() override;

	virtual bool Send(const uint8* Data, const uint64 DataSize) override;
	virtual bool Recv(uint8* Data, const uint64 DataSize, uint64& BytesRead, ESocketReceiveFlags::Type ReceiveFlags) override;
	virtual bool HasPendingData(uint64& PendingDataSize) const override;
	virtual void Close() override;
	virtual bool IsAlive() const override;

	int32 GetProtocolNumber() const { return ProtocolNumber; }

private:
	IPlatformHostCommunication* Communication;
	IPlatformHostSocketPtr Socket;
	FConnectionCircularBuffer ConnectionBuffer;
	const int32 ProtocolNumber;
};

class FBuiltInHttpClientPlatformSocketPool : public IBuiltInHttpClientSocketPool
{
public:
	FBuiltInHttpClientPlatformSocketPool(const FString InAddress);
	virtual ~FBuiltInHttpClientPlatformSocketPool() override;

	virtual IBuiltInHttpClientSocket* AcquireSocket(float TimeoutSeconds = -1.f) override;
	virtual void ReleaseSocket(IBuiltInHttpClientSocket* Socket, bool bKeepAlive) override;

private:
	const FString Address;
	IPlatformHostCommunication* Communication = nullptr;

	TLockFreePointerListUnordered<IBuiltInHttpClientSocket, PLATFORM_CACHE_LINE_SIZE> SocketPool;

	FCriticalSection UsedSocketsCS;
	UE::FConditionVariable UsedSocketsCV;
	TBitArray<> UsedSockets; // bitset to keep track of used sockets in the pool
};

#endif