// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuiltInHttpClientUDSSocket.h"
#if !UE_BUILD_SHIPPING && PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS
#include "StorageServerConnection.h"
#include "Windows/AllowWindowsPlatformTypes.h"

#include <winsock2.h>
#include <ws2tcpip.h>
#include <afunix.h>

// MSG_WAITALL is 0x8 on Windows, but 0x100 on other platforms.
// NOTE: copies low-level TranslateFlags from Sockets\Private\SocketSubsystemBSDPrivate.h.
inline int TranslateFlags(ESocketReceiveFlags::Type Flags)
{
	int TranslatedFlags = 0;

	if (Flags & ESocketReceiveFlags::Peek)
	{
		TranslatedFlags |= MSG_PEEK;
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_MSG_DONTWAIT
		TranslatedFlags |= MSG_DONTWAIT;
#endif // PLATFORM_HAS_BSD_SOCKET_FEATURE_MSG_DONTWAIT
	}

	if (Flags & ESocketReceiveFlags::WaitAll)
	{
		TranslatedFlags |= MSG_WAITALL;
	}

	return TranslatedFlags;
}

FBuiltInHttpClientUDSSocket::FBuiltInHttpClientUDSSocket(uint64 InSocket)
	: Socket(InSocket)
{
}

FBuiltInHttpClientUDSSocket::~FBuiltInHttpClientUDSSocket()
{
	if (Socket != INVALID_SOCKET)
	{
		closesocket(Socket);
	}
}
bool FBuiltInHttpClientUDSSocket::Send(
	const uint8* Data,
	const uint64 DataSize)
{
	if (Socket == INVALID_SOCKET)
	{
		return false;
	}

	uint64 TotalBytesSent = 0;
	while (TotalBytesSent < DataSize)
	{
		const uint64 BytesRemaining = DataSize - TotalBytesSent;
		const int32 BytesToSend = (int32)FMath::Min<uint64>(BytesRemaining, (uint64)INT32_MAX);
		int32 BytesSent = send(Socket, (const char*)(Data + TotalBytesSent), BytesToSend, 0);
		if (BytesSent <= 0)
		{
			// send() returning 0 on a Winsock stream socket indicates the peer has
			// closed the connection or no forward progress is possible; either way,
			// continuing the loop would spin forever sending zero bytes.
			return false;
		}
		TotalBytesSent += (uint64)BytesSent;
	}

	return true;
}

bool FBuiltInHttpClientUDSSocket::Recv(
	uint8* Data,
	const uint64 DataSize,
	uint64& BytesRead,
	ESocketReceiveFlags::Type ReceiveFlags)
{
	BytesRead = 0;

	if (Socket == INVALID_SOCKET)
	{
		return false;
	}
	
	if (DataSize > INT_MAX)
	{
		return false;
	}

	int ReadBytes = recv(Socket, (char*)Data, (int)DataSize, TranslateFlags(ReceiveFlags));

	// 0 means the socket is closed gracefully no more data incoming. 
	if (ReadBytes >= 0)
	{
		BytesRead = (uint64)ReadBytes;
		return true;
	}
	
	ReadBytes = 0;

	int Code = WSAGetLastError();
	if (Code == WSAEWOULDBLOCK)
		return true;

	return false;
}

bool FBuiltInHttpClientUDSSocket::HasPendingData(
	uint64& PendingDataSize) const
{
	PendingDataSize = 0;
	if (Socket == INVALID_SOCKET)
	{
		return false;
	}

#if PLATFORM_HAS_BSD_SOCKET_FEATURE_IOCTL
	// See if there is any pending data on the read socket
	uint32 DataAvailable = 0;
	if (ioctlsocket(Socket, FIONREAD, (u_long*)(&DataAvailable)) == 0)
	{
		PendingDataSize = DataAvailable;
		return (PendingDataSize > 0);
	}
	else
	{
		UE_LOGF(LogStorageServerConnection, Fatal, "Call to obtain the pending data on socket failed");
	}
#endif

	return false;
}

void FBuiltInHttpClientUDSSocket::Close()
{
	if (Socket != INVALID_SOCKET)
	{
		closesocket(Socket);
		Socket = INVALID_SOCKET;
	}
}

bool FBuiltInHttpClientUDSSocket::IsAlive() const
{
	if (Socket == INVALID_SOCKET)
	{
		return false;
	}

	// Use select() with a zero timeout to determine readability. A correctly-drained
	// pooled keep-alive socket should report not-readable; any other outcome means we
	// can't safely reuse it:
	//   - readable + EOF (peer closed) - reuse would cause a hang on the next recv()
	//   - readable + bytes pending (previous response not fully drained) - reuse would
	//     corrupt the next response stream
	//   - readable for any other reason - rare, and conservatively also discarded
	// We deliberately don't peek to distinguish EOF from pending bytes - both cases
	// must be dropped, so the extra recv(MSG_PEEK) round-trip would only add cost.
	fd_set ReadSet;
	FD_ZERO(&ReadSet);
	FD_SET(Socket, &ReadSet);

	timeval ZeroTimeout = { 0, 0 };
	const int SelectResult = select(0, &ReadSet, nullptr, nullptr, &ZeroTimeout);

	if (SelectResult == 0)
	{
		// Not readable: connection open and idle - the expected state for a pooled socket.
		return true;
	}

	// Either select() failed or the socket is readable; either way it is not safe to
	// reuse. Caller will close and create a fresh connection.
	return false;
}

FBuiltInHttpClientUDSSocketPool::FBuiltInHttpClientUDSSocketPool(
	FString InServerAddr)
	: Address(InServerAddr)
{
}

FBuiltInHttpClientUDSSocketPool::~FBuiltInHttpClientUDSSocketPool()
{
	IBuiltInHttpClientSocket* Socket = nullptr;
	while ((Socket = SocketPool.Pop()) != nullptr)
	{
		delete Socket;
	}
}

IBuiltInHttpClientSocket* FBuiltInHttpClientUDSSocketPool::AcquireSocket(
	float TimeoutSeconds)
{
	// Drain stale sockets out of the pool. A busy server commonly reaps idle keep-alive
	// connections, leaving us holding peer-closed sockets that look fine until we try to
	// recv from them; check liveness explicitly before reuse.
	while (IBuiltInHttpClientSocket* Pooled = SocketPool.Pop())
	{
		if (Pooled->IsAlive())
		{
			return Pooled;
		}
		UE_LOGF(LogStorageServerConnection, Verbose, "Discarded stale pooled UDS socket");
		delete Pooled;
	}

	SOCKET ListenSocket = socket(AF_UNIX, SOCK_STREAM, 0);
	if (ListenSocket == INVALID_SOCKET)
	{
		UE_LOGF(LogStorageServerConnection, Error, "failed to create AF_UNIX socket");
		return nullptr;
	}

	SOCKADDR_UN SocketAddress = { 0 };
	SocketAddress.sun_family = AF_UNIX;
	TExternalStringBuilder<char> Builder(SocketAddress.sun_path, UNIX_PATH_MAX);
	Builder.Append(*Address, Address.Len());

	bool bConnected = false;
	int ConnectError = 0;

	if (TimeoutSeconds > 0.0f)
	{
		// Non-blocking connect bounded by select() so a busy server with a full accept
		// backlog doesn't leave us hanging on connect().
		uint32 NonBlocking = 1;
		if (ioctlsocket(ListenSocket, FIONBIO, (u_long*)(&NonBlocking)) == 0)
		{
			const int InitialResult = connect(ListenSocket, (struct sockaddr*)&SocketAddress, sizeof(SOCKADDR_UN));
			if (InitialResult == 0)
			{
				bConnected = true;
			}
			else
			{
				ConnectError = WSAGetLastError();
				if (ConnectError == WSAEWOULDBLOCK || ConnectError == WSAEINPROGRESS)
				{
					fd_set WriteSet;
					FD_ZERO(&WriteSet);
					FD_SET(ListenSocket, &WriteSet);

					fd_set ErrorSet;
					FD_ZERO(&ErrorSet);
					FD_SET(ListenSocket, &ErrorSet);

					timeval Timeout;
					Timeout.tv_sec = (long)TimeoutSeconds;
					Timeout.tv_usec = (long)((TimeoutSeconds - (float)Timeout.tv_sec) * 1000000.0f);

					const int SelectResult = select(0, nullptr, &WriteSet, &ErrorSet, &Timeout);
					if (SelectResult > 0 && FD_ISSET(ListenSocket, &WriteSet))
					{
						int SockError = 0;
						int SockErrorLen = sizeof(SockError);
						if (getsockopt(ListenSocket, SOL_SOCKET, SO_ERROR, (char*)&SockError, &SockErrorLen) == 0 && SockError == 0)
						{
							bConnected = true;
						}
						else
						{
							ConnectError = SockError;
						}
					}
					else if (SelectResult == 0)
					{
						ConnectError = WSAETIMEDOUT;
					}
					else
					{
						ConnectError = WSAGetLastError();
					}
				}
			}

			// Restore blocking mode for the request/response phase. If this fails the
			// socket would be left non-blocking, and our Send()/Recv() expect blocking
			// semantics (with SO_SNDTIMEO/SO_RCVTIMEO providing the deadline). A stray
			// WSAEWOULDBLOCK from send() would be misread as a fatal error, so treat a
			// failed restore the same as a failed connect.
			uint32 Blocking = 0;
			if (ioctlsocket(ListenSocket, FIONBIO, (u_long*)(&Blocking)) != 0)
			{
				if (bConnected)
				{
					ConnectError = WSAGetLastError();
					bConnected = false;
				}
			}
		}
		else
		{
			ConnectError = WSAGetLastError();
		}
	}
	else
	{
		if (connect(ListenSocket, (struct sockaddr*)&SocketAddress, sizeof(SOCKADDR_UN)) == 0)
		{
			bConnected = true;
		}
		else
		{
			ConnectError = WSAGetLastError();
		}
	}

	if (!bConnected)
	{
		UE_LOGF(LogStorageServerConnection, Error, "connect failed: %ls (WSA error %d)", *Address, ConnectError);
		closesocket(ListenSocket);
		return nullptr;
	}

	return new FBuiltInHttpClientUDSSocket(ListenSocket);
}

void FBuiltInHttpClientUDSSocketPool::ReleaseSocket(
	IBuiltInHttpClientSocket* Socket,
	bool bKeepAlive)
{
	uint64 PendingDataSize = 0;
	if (bKeepAlive && !Socket->HasPendingData(PendingDataSize))
	{
		SocketPool.Push(Socket);
	}
	else
	{
		if (PendingDataSize > 0)
		{
			UE_LOGF(LogStorageServerConnection, Fatal, "Socket was not fully drained");
		}

		delete Socket;
	}
}

#include "Windows/HideWindowsPlatformTypes.h"
#endif // !UE_BUILD_SHIPPING && PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS
