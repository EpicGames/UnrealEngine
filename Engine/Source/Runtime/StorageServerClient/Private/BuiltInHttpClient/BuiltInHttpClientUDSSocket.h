// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BuiltInHttpClient.h"
#include "Containers/LockFreeList.h"

#if !UE_BUILD_SHIPPING && PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS

class FBuiltInHttpClientUDSSocket : public IBuiltInHttpClientSocket
{
public:
	FBuiltInHttpClientUDSSocket(uint64 InSocket);
	virtual ~FBuiltInHttpClientUDSSocket() override;

	virtual bool Send(
		const uint8* Data, 
		const uint64 DataSize) override;

	virtual bool Recv(
		uint8* Data, 
		const uint64 DataSize, 
		uint64& BytesRead, 
		ESocketReceiveFlags::Type ReceiveFlags) override;

	virtual bool HasPendingData(
		uint64& PendingDataSize) const override;

	virtual void Close() override;

	virtual bool IsAlive() const override;

private:
	uint64 Socket;
};

class FBuiltInHttpClientUDSSocketPool : public IBuiltInHttpClientSocketPool
{
public:
	FBuiltInHttpClientUDSSocketPool(FString InServerAddr);
	virtual ~FBuiltInHttpClientUDSSocketPool() override;

	virtual IBuiltInHttpClientSocket* AcquireSocket(
		float TimeoutSeconds = -1.f) override;
	
	virtual void ReleaseSocket(
		IBuiltInHttpClientSocket* Socket, 
		bool bKeepAlive) override;

private:
	const FString Address;
	TLockFreePointerListUnordered<IBuiltInHttpClientSocket, PLATFORM_CACHE_LINE_SIZE> SocketPool;
};

#endif