// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/IoChunkId.h"
#include "IO/IoHash.h"
#include "Templates/Function.h"
#include "Containers/Array.h"
#include "Math/Interval.h"
#include "StorageServerHttpClient.h"

#if !UE_BUILD_SHIPPING

namespace StorageServer
{
	struct FCacheChunkInfo
	{
		TOptional<uint64> ModTag;      // content modification tag
		TOptional<int64> RawSize;      // size of complete chunk
		TOptional<int32> RawBlockSize; // size of block in a chunk if any, e.g. 256kb, can be 0

		bool SetChunkInfo(
			const TOptional<uint64>& OptModTag,
			const TOptional<int64>& OptRawSize,
			const TOptional<int32>& OptRawBlockSize
		)
		{
			bool bResult = true;
			if (OptModTag)
			{
				bResult = ModTag.IsSet() ? *ModTag == *OptModTag : true;
				ModTag = *OptModTag;
			}
			if (OptRawSize)
			{
				RawSize = *OptRawSize;
			}
			if (OptRawBlockSize)
			{
				RawBlockSize = *OptRawBlockSize;
			}
			return bResult;
		}

		FCacheChunkInfo& operator=(const FCacheChunkInfo& Other)
		{
			ModTag = Other.ModTag;
			RawSize = Other.RawSize;
			RawBlockSize = Other.RawBlockSize;
			return *this;
		}

		friend FArchive& operator << (FArchive& Ar, FCacheChunkInfo& Info)
		{
			Ar << Info.ModTag;
			Ar << Info.RawSize;
			Ar << Info.RawBlockSize;
			return Ar;
		}
	};

	struct FCacheEntry
	{
		uint64 ChunkOffset;   // uncompressed offset in a chunk
		uint64 ChunkSize;     // uncompressed size in a chunk
		uint64 StorageOffset; // offset in cache storage
		uint64 StorageSize;   // size in cache storage
		FIoHash StorageHash;  // hash of data in cache storage, needed to validate after reading from storage as we might lose data
		EStorageServerContentType StorageContentType; // content type in storage

		TInterval<uint64> GetChunkInterval() const
		{
			return TInterval(ChunkOffset, ChunkSize);
		}

		friend FArchive& operator << (FArchive& Ar, FCacheEntry& Entry)
		{
			Ar << Entry.ChunkOffset;
			Ar << Entry.ChunkSize;
			Ar << Entry.StorageOffset;
			Ar << Entry.StorageSize;
			Ar << Entry.StorageHash;
			Ar << Entry.StorageContentType;
			return Ar;
		}
	};

	// Generic journal for storage server caching. Transactions are thread safe, atomic, saved at best effort.
	// Cache journal focuses on avoiding data corruption when saving to disk, unlike cache storage which doesn't provide any guarantees for data consistency.
	class ICacheJournal
	{
	public:
		virtual ~ICacheJournal() = default;

		// Flushes data to backing storage
		virtual void Flush(bool bImmediate) = 0;

		// Invalidates all data in journal
		virtual void InvalidateAll() = 0;

		// Invalidates data for a specific chunk.
		virtual void Invalidate(const FIoChunkId& ChunkId) = 0;

		// Updates chunk info. All fields are optional.
		// If OptModTag is passed will compare it to existing ModTag and asks you to invalidate all data for this chunk id if they don't match.
		// OptRawSize, OptRawBlockSize will simply be updated if passed.
		// Returns true if either new ChunkId info is created or ModTag matches previous existing entry for this ChunkId.
		// Returns false if ModHash doesn't match and cached data for this entry needs to be invalidated.
		virtual bool SetChunkInfo(
			const FIoChunkId& ChunkId,
			const TOptional<uint64>& OptModTag,
			const TOptional<int64>& OptRawSize,
			const TOptional<int32>& OptRawBlockSize
		) = 0;

		// Return chunk info if present, returns true if chunk info is present.
		virtual bool TryGetChunkInfo(
			const FIoChunkId& ChunkId,
			FCacheChunkInfo& OutChunkInfo
		) = 0;

		// Adds a new cache entry for chunk.
		// Returns true if new entry was added, false if there is an entry for (ChunkId / ChunkOffset / ChunkSize).
		virtual bool AddEntry(
			const FIoChunkId& ChunkId,
			const FCacheEntry& Entry
		) = 0;

		// Tries to find an entry for specified chunk offset and size, returns true if entry is present.
		virtual bool TryGetEntry(
			const FIoChunkId& ChunkId,
			const uint64 ChunkOffset,
			const uint64 ChunkSize,
			FCacheEntry& OutEntry
		) = 0;

		// Iterate chunk id in journal that contain at least one entry.
		// Not safe to invoke other journal methods from the callback.
		virtual void IterateChunkIds(
			TFunctionRef<void(const FIoChunkId& ChunkId, const FCacheChunkInfo& ChunkInfo)> Callback
		) = 0;

		// Iterate entries for a chunk id in journal.
		// Not safe to invoke other journal methods from the callback.
		virtual void IterateCacheEntriesForChunkId(
			const FIoChunkId& ChunkId,
			TFunctionRef<void(const FCacheEntry& Entry)> Callback
		) = 0;

		// Iterate all entries in journal.
		// Not safe to invoke other journal methods from the callback.
		virtual void IterateCacheEntries(
			TFunctionRef<void(const FIoChunkId& ChunkId, const FCacheEntry& Entry)> Callback
		) = 0;
	};
}

#endif