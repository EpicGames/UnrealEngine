// Copyright Epic Games, Inc. All Rights Reserved.

#include "CacheJournalSectioned.h"

#if !UE_BUILD_SHIPPING

#include "HAL/PlatformFile.h"
#include "Misc/ScopeLock.h"
#include "Serialization/MemoryWriter.h"
#include "ProfilingDebugging/CountersTrace.h"

DEFINE_LOG_CATEGORY_STATIC(LogCacheJournal, Log, All);

TRACE_DECLARE_INT_COUNTER(ZenJournalSectionedChunks, TEXT("ZenClient/JournalSectioned/Chunks"));
TRACE_DECLARE_INT_COUNTER(ZenJournalSectionedEntries, TEXT("ZenClient/JournalSectioned/Entries"));

namespace StorageServer
{
	static inline constexpr uint64 FILE_MAGIC = 0x5a45'4e43; // ZENC
	static inline constexpr uint64 PAGE_MAGIC = 0x5041'4745; // PAGE
	static inline constexpr uint32 JOURNAL_VERSION = 0x03;

	// TODO hardcoded constants of max entries are fragile, can we compute it from page size directly?
	static inline constexpr uint32 GMaxChunkInfoEntries = 2950; // approx 44 bytes per entry
	static inline constexpr uint32 GMaxChunkEntries = 1880; // approx 66 bytes per entry
	static inline constexpr uint32 JOURNAL_PAGE_SIZE = 128 * 1024;

	FCacheJournalSectioned::FCacheJournalSectioned(FStringView InJournalPath)
		: JournalFilePath(InJournalPath)
	{
		JournalFileHandle.Reset(IPlatformFile::GetPlatformPhysical().OpenWrite(*JournalFilePath, true, true));

		// If loading the journal failed, invalidate all of the data
		if (!LoadCacheJournal())
		{
			InvalidateAll();
		}
		else
		{
			// If the journal loaded, there still might be entries without corresponding cache chunk infos. Validate them
			ValidateJournal();
		}
	}

	void FCacheJournalSectioned::Flush(bool bImmediate)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FCacheJournalSectioned::Flush);

		if (bImmediate)
		{
			FlushImmediate();
		}
		else
		{
			UE::TScopeLock _(CriticalSection);

			if (FlushTask.IsCompleted())
			{
				FlushTask = UE::Tasks::Launch(TEXT("CacheJournalFlush"), [this]() {FlushImmediate();});
			}
		}
	}

	void FCacheJournalSectioned::FlushImmediate()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FCacheJournalSectioned::FlushImmediate);

		UE::TScopeLock _(CriticalSection);

		TArray<uint8> SerializationBuffer;

		bool bNeedsFlush = false;

		if (bPagesModified)
		{
			FJournalHeader Header;
			Header.Magic = FILE_MAGIC;
			Header.Version = JOURNAL_VERSION;
			Header.PageCount = ChunkPages.Num() + ChunkInfoPages.Num();

			SerializationBuffer.SetNum(0, EAllowShrinking::No);
			FMemoryWriter Ar(SerializationBuffer);
			Ar << Header;

			if (!ensure(SerializationBuffer.Num() <= FJournalHeader::SerializedSize))
			{
				return;
			}

			if (!JournalFileHandle->Seek(0))
			{
				return;
			}

			if (!JournalFileHandle->Write(SerializationBuffer.GetData(), SerializationBuffer.Num()))
			{
				return;
			}

			bPagesModified = false;
			bNeedsFlush = true;
		}

		for (FJournalChunkPage& Page : ChunkPages)
		{
			bNeedsFlush |= Page.Flush(JournalFileHandle.Get(), SerializationBuffer);
		}

		for (FJournalChunkInfoPage& Page : ChunkInfoPages)
		{
			bNeedsFlush |= Page.Flush(JournalFileHandle.Get(), SerializationBuffer);
		}

		if (bNeedsFlush)
		{
			JournalFileHandle->Flush(true);
		}
	}

	void FCacheJournalSectioned::InvalidateAll()
	{
		UE::TScopeLock _(CriticalSection);

		ChunkPages.Empty();
		ChunkInfoPages.Empty();
		ResetJournalFilePos();

		TRACE_COUNTER_SET(ZenJournalSectionedChunks, 0);
		TRACE_COUNTER_SET(ZenJournalSectionedEntries, 0);

		bPagesModified = true;
	}

	void FCacheJournalSectioned::Invalidate(const FIoChunkId& ChunkId)
	{
		UE::TScopeLock _(CriticalSection);

		for (FJournalChunkPage& Page : ChunkPages)
		{
			Page.Invalidate(ChunkId);
		}

		for (FJournalChunkInfoPage& InfoPage : ChunkInfoPages)
		{
			InfoPage.Invalidate(ChunkId);
		}
	}

	bool FCacheJournalSectioned::SetChunkInfo(const FIoChunkId& ChunkId, const TOptional<uint64>& OptModHash, const TOptional<int64>& OptRawSize, const TOptional<int32>& OptRawBlockSize)
	{
		UE::TScopeLock _(CriticalSection);

		FCacheChunkInfo ChunkInfo;
		for (FJournalChunkInfoPage& Page : ChunkInfoPages)
		{
			if (Page.GetEntry(ChunkId, ChunkInfo))
			{
				const bool bHashesMatch = ChunkInfo.SetChunkInfo(OptModHash, OptRawSize, OptRawBlockSize);
				Page.Add(ChunkId, ChunkInfo);
				return bHashesMatch;
			}
		}

		FJournalChunkInfoPage* Page = FindOrAddChunkInfoPage(ChunkId);
		check(Page != nullptr && !Page->IsFull());

		ChunkInfo.SetChunkInfo(OptModHash, OptRawSize, OptRawBlockSize);
		Page->Add(ChunkId, ChunkInfo);
		TRACE_COUNTER_INCREMENT(ZenJournalSectionedEntries);
		return true;
	}

	bool FCacheJournalSectioned::TryGetChunkInfo(const FIoChunkId& ChunkId, FCacheChunkInfo& OutChunkInfo)
	{
		UE::TScopeLock _(CriticalSection);

		for (FJournalChunkInfoPage& Page : ChunkInfoPages)
		{
			if (Page.GetEntry(ChunkId, OutChunkInfo))
			{
				return true;
			}
		}

		return false;
	}

	bool FCacheJournalSectioned::AddEntry(const FIoChunkId& ChunkId, const FCacheEntry& Entry)
	{
		UE::TScopeLock _(CriticalSection);

		FCacheEntry OutEntry;
		if (TryGetEntry(ChunkId, Entry.ChunkOffset, Entry.ChunkSize, OutEntry))
		{
			return false;
		}

		FJournalChunkPage* Page = FindOrAddChunkPage(ChunkId);
		check(Page != nullptr && !Page->IsFull());

		EJournalPageResult Res = Page->Add(ChunkId, Entry);
		return Res == EJournalPageResult::Ok;
	}

	bool FCacheJournalSectioned::TryGetEntry(const FIoChunkId& ChunkId, const uint64 ChunkOffset, const uint64 ChunkSize, FCacheEntry& OutEntry)
	{
		UE::TScopeLock _(CriticalSection);

		for (FJournalChunkPage& Page : ChunkPages)
		{
			if (Page.GetEntry(ChunkId, ChunkOffset, ChunkSize, OutEntry))
			{
				return true;
			}
		}

		return false;
	}

	void FCacheJournalSectioned::IterateChunkIds(TFunctionRef<void(const FIoChunkId& ChunkId, const FCacheChunkInfo& ChunkInfo)> Callback)
	{
		UE::TScopeLock _(CriticalSection);

		for (FJournalChunkInfoPage& Page : ChunkInfoPages)
		{
			Page.IterateChunkIds(MoveTemp(Callback));
		}
	}

	void FCacheJournalSectioned::IterateCacheEntriesForChunkId(const FIoChunkId& ChunkId, TFunctionRef<void(const FCacheEntry& Entry)> Callback)
	{
		UE::TScopeLock _(CriticalSection);

		for (FJournalChunkPage& Page : ChunkPages)
		{
			Page.IterateCacheEntriesForChunkId(ChunkId, MoveTemp(Callback));
		}
	}

	void FCacheJournalSectioned::IterateCacheEntries(TFunctionRef<void(const FIoChunkId& ChunkId, const FCacheEntry& Entry)> Callback)
	{
		UE::TScopeLock _(CriticalSection);

		for (FJournalChunkPage& Page : ChunkPages)
		{
			Page.IterateCacheEntries(MoveTemp(Callback));
		}
	}

	bool FCacheJournalSectioned::LoadCacheJournal()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FCacheJournalSectioned::LoadCacheJournal);

		UE::TScopeLock _(CriticalSection);

		if (!JournalFileHandle.IsValid() || JournalFileHandle->Size() < FJournalHeader::SerializedSize)
		{
			return false;
		}

		TArray<uint8> SerializationBuffer;
		SerializationBuffer.Reserve(JOURNAL_PAGE_SIZE);

		FJournalHeader Header;

		{
			SerializationBuffer.SetNum(FJournalHeader::SerializedSize, EAllowShrinking::No);

			if (!JournalFileHandle->ReadAt(SerializationBuffer.GetData(), SerializationBuffer.Num(), 0))
			{
				return false;
			}

			FMemoryReader HeaderAr(SerializationBuffer);
			HeaderAr << Header;

			if (Header.Magic != FILE_MAGIC)
			{
				UE_LOGF(LogCacheJournal, Warning, "Journal file has wrong magic number, the journal is corrupt");
				InvalidateAll();
				return false;
			}

			if (Header.Version != JOURNAL_VERSION)
			{
				UE_LOGF(LogCacheJournal, Warning, "Journal file is of different version %d, supported %d", Header.Version, JOURNAL_VERSION);
				InvalidateAll();
				return false;
			}
		}

		int64 ReadCursor = FJournalHeader::SerializedSize;
		uint64 PageCount = 0;

		uint64 ChunkInfoEntryCount = 0;
		uint64 ChunkEntryCount = 0;

		while ((ReadCursor + FJournalPageHeader::SerializedSize < JournalFileHandle->Size()) && (PageCount < Header.PageCount))
		{
			const int64 PageOffset = ReadCursor;

			FJournalPageHeader PageHeader;
			SerializationBuffer.SetNum(FJournalPageHeader::SerializedSize, EAllowShrinking::No);

			if (!JournalFileHandle->ReadAt(SerializationBuffer.GetData(), SerializationBuffer.Num(), PageOffset))
			{
				UE_LOGF(LogCacheJournal, Warning, "Failed to read page header from section in journal file");
				return false;
			}

			FMemoryReader HeaderAr(SerializationBuffer);
			HeaderAr << PageHeader;

			if (PageHeader.Magic != PAGE_MAGIC)
			{
				UE_LOGF(LogCacheJournal, Warning, "Section has wrong magic number, the journal is corrupt");
				InvalidateAll();
				return false;
			}

			if (PageHeader.PageSize > JOURNAL_PAGE_SIZE || PageHeader.DataSize >= PageHeader.PageSize)
			{
				UE_LOGF(LogCacheJournal, Warning, "Section has wrong size, the journal is corrupt");
				InvalidateAll();
				return false;
			}

			SerializationBuffer.SetNum(PageHeader.DataSize, EAllowShrinking::No);

			if (!JournalFileHandle->ReadAt(SerializationBuffer.GetData(), SerializationBuffer.Num(), PageOffset + FJournalPageHeader::SerializedSize))
			{
				UE_LOGF(LogCacheJournal, Warning, "Failed to read page from section in journal file.");
				InvalidateAll();
				return false;
			}

			const FIoHash DataHash = FIoHash::HashBuffer(SerializationBuffer.GetData(), SerializationBuffer.Num());

			if (DataHash != PageHeader.DataHash)
			{
				UE_LOGF(LogCacheJournal, Warning, "Page has invalid data hash, the journal is corrupt.");
				InvalidateAll();
				return false;
			}

			FJournalPageBase* Page = nullptr;
			switch (PageHeader.Type)
			{
			case EPageType::Chunk:
				Page = &ChunkPages.Add_GetRef({ PageHeader.PageSize, GMaxChunkEntries }); // TODO GMaxChunkEntries ?
				break;
			case EPageType::ChunkInfo:
				Page = &ChunkInfoPages.Add_GetRef({ PageHeader.PageSize, GMaxChunkInfoEntries }); // TODO GMaxChunkInfoEntries ?
				break;
			default:
				UE_LOGF(LogCacheJournal, Warning, "Unknown page type in journal file.");
				InvalidateAll();
				return false;
			}

			Page->SetFilePos(PageOffset);

			FMemoryReader BodyAr(SerializationBuffer);
			BodyAr << *Page;

			ReadCursor += PageHeader.PageSize;
			PageCount++;

			switch (PageHeader.Type)
			{
			case EPageType::Chunk:
				ChunkEntryCount += ((FJournalChunkPage*)Page)->GetEntryCount();
				break;
			case EPageType::ChunkInfo:
				ChunkInfoEntryCount += ((FJournalChunkInfoPage*)Page)->GetEntryCount();
				break;
			default:
				break;
			}
		}

		TRACE_COUNTER_SET(ZenJournalSectionedChunks, ChunkEntryCount);
		TRACE_COUNTER_SET(ZenJournalSectionedEntries, ChunkInfoEntryCount);

		UE_LOG(LogCacheJournal, Display, TEXT("Cache journal loaded. Imported %d chunk pages with %" UINT64_FMT " entries and %d chunk info pages with %" UINT64_FMT " info entries."), ChunkPages.Num(), ChunkEntryCount, ChunkInfoPages.Num(), ChunkInfoEntryCount);

		return true;
	}

	void FCacheJournalSectioned::ValidateJournal()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FCacheJournalSectioned::ValidateJournal);

		UE::TScopeLock _(CriticalSection);

		TArray<FIoChunkId> InvalidChunkIds;
		auto ValidateEntryInfo = [&InvalidChunkIds, this](const FIoChunkId& ChunkId, const FCacheEntry& CacheEntry)
			{
				FCacheChunkInfo ChunkInfo;
				if (!TryGetChunkInfo(ChunkId, ChunkInfo))
				{
					InvalidChunkIds.Add(ChunkId);
				}
			};
		
		TSet<FIoChunkId> ChunkIdSet;
		TArray<FIoChunkId> DuplicateChunkIds;
		auto ValidateChunkId = [&](const FIoChunkId& ChunkId, const FCacheChunkInfo& ChunkInfo)
			{
				const uint8* ChunkdIdBytes = ChunkId.GetData();
				const int32 IdByteCount = ChunkId.GetSize() - sizeof(decltype(ChunkId.GetChunkType()));

				if (!FMemory::Memcmp(ChunkdIdBytes, FIoChunkId::InvalidChunkId.GetData(), IdByteCount))
				{
					InvalidChunkIds.Add(ChunkId);
				}

				if (ChunkIdSet.Contains(ChunkId))
				{
					DuplicateChunkIds.Add(ChunkId);
				}
				else
				{
					ChunkIdSet.Add(ChunkId);
				}
			};
		
		IterateChunkIds(ValidateChunkId);
		IterateCacheEntries(ValidateEntryInfo);
		for (const FIoChunkId& ChunkId : InvalidChunkIds)
		{
			Invalidate(ChunkId);
		}

		for (const FIoChunkId& ChunkId : DuplicateChunkIds)
		{
			Invalidate(ChunkId);
		}

		if (InvalidChunkIds.Num() > 0)
		{
			UE_LOGF(LogCacheJournal, Warning, "There were %d invalid zen cache journal entries.", InvalidChunkIds.Num());
		}

		if (DuplicateChunkIds.Num() > 0)
		{
			UE_LOGF(LogCacheJournal, Warning, "There were %d duplicate chunk id entries.", DuplicateChunkIds.Num());
		}
	}

	FJournalChunkInfoPage* FCacheJournalSectioned::FindOrAddChunkInfoPage(const FIoChunkId& ChunkId)
	{
		for (FJournalChunkInfoPage& Page : ChunkInfoPages)
		{
			check(!Page.Contains(ChunkId));
		}

		for (FJournalChunkInfoPage& Page : ChunkInfoPages)
		{
			if (!Page.IsFull())
			{
				return &Page;
			}
		}

		return AddChunkInfoPage();
	}

	FJournalChunkPage* FCacheJournalSectioned::FindOrAddChunkPage(const FIoChunkId& ChunkId)
	{
		FJournalChunkPage* LastPage = nullptr;
		for (FJournalChunkPage& Page : ChunkPages)
		{
			if (!Page.IsFull())
			{
				LastPage = &Page;

				if (Page.Contains(ChunkId))
				{
					return &Page;
				}
			}
		}

		return LastPage ? LastPage : AddChunkPage();
	}

	FJournalChunkInfoPage* FCacheJournalSectioned::AddChunkInfoPage()
	{
		FJournalChunkInfoPage* Page = &ChunkInfoPages.Add_GetRef({ JOURNAL_PAGE_SIZE, GMaxChunkInfoEntries });
		Page->SetFilePos(AllocateJournalFilePos(Page->GetPageSize()));
		bPagesModified = true;
		return Page;
	}

	FJournalChunkPage* FCacheJournalSectioned::AddChunkPage()
	{
		FJournalChunkPage* Page = &ChunkPages.Add_GetRef({ JOURNAL_PAGE_SIZE, GMaxChunkEntries });
		Page->SetFilePos(AllocateJournalFilePos(Page->GetPageSize()));
		bPagesModified = true;
		return Page;
	}

	bool FJournalPageBase::Flush(IFileHandle* JournalFile, TArray<uint8>& SerializationBuffer)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FJournalPageBase::Flush);

		if (!bDirty)
		{
			return false; // don't require flush if nothing changed
		}

		SerializationBuffer.SetNum(0, EAllowShrinking::No);
		FMemoryWriter BodyAr(SerializationBuffer);
		BodyAr << *this;

		if (!ensureMsgf(SerializationBuffer.Num() + FJournalPageHeader::SerializedSize <= PageSize, TEXT("Page %u serialized to %i bytes > max size of %" INT64_FMT), EnumToUnderlyingType(PageType), SerializationBuffer.Num() + FJournalPageHeader::SerializedSize, PageSize))
		{
			return false;
		}

		FJournalPageHeader Descriptor;
		Descriptor.Magic = PAGE_MAGIC;
		Descriptor.Type = PageType;
		Descriptor.PageSize = PageSize;
		Descriptor.DataSize = SerializationBuffer.Num();
		Descriptor.DataHash = FIoHash::HashBuffer(SerializationBuffer.GetData(), SerializationBuffer.Num());

		TArray<uint8, TInlineAllocator<128>> HeaderBuffer; // need to write payload size into header, so avoid using same buffer
		TMemoryWriterBase HeaderAr(HeaderBuffer);
		HeaderAr << Descriptor;

		if (!ensure(HeaderBuffer.Num() <= FJournalPageHeader::SerializedSize))
		{
			return false;
		}

		if (!JournalFile->Seek(FilePos))
		{
			return false;
		}

		if (!JournalFile->Write(HeaderBuffer.GetData(), HeaderBuffer.Num()))
		{
			return false;
		}

		if (!JournalFile->Seek(FilePos + FJournalPageHeader::SerializedSize))
		{
			return false;
		}

		if (!JournalFile->Write(SerializationBuffer.GetData(), SerializationBuffer.Num()))
		{
			return false;
		}

		bDirty = false;
		return true;
	}

	FJournalChunkInfoPage::FJournalChunkInfoPage(int64 InPageSize, int32 InMaxEntries)
		: FJournalPageBase(EPageType::ChunkInfo, InPageSize)
		, MaxEntryCount(InMaxEntries)
	{
	}

	EJournalPageResult FJournalChunkInfoPage::Add(const FIoChunkId& ChunkId, const FCacheChunkInfo& Entry)
	{
		FCacheChunkInfo* ChunkInfo = Entries.Find(ChunkId);
		if (ChunkInfo)
		{
			*ChunkInfo = Entry;
			return EJournalPageResult::EntryAlreadyExists;
		}

		if (Entries.Num() >= MaxEntryCount)
		{
			return EJournalPageResult::PageFull;
		}

		Entries.Add(ChunkId, Entry);
		bDirty = true;
		return EJournalPageResult::Ok;
	}

	bool FJournalChunkInfoPage::GetEntry(const FIoChunkId& ChunkId, FCacheChunkInfo& Info)
	{
		FCacheChunkInfo* ChunkInfo = Entries.Find(ChunkId);
		if (!ChunkInfo)
		{
			return false;
		}

		Info = *ChunkInfo;
		return true;
	}

	bool FJournalChunkInfoPage::Contains(const FIoChunkId& ChunkId) const
	{
		return Entries.Contains(ChunkId);
	}

	void FJournalChunkInfoPage::Invalidate(const FIoChunkId& ChunkId)
	{
		const bool bRemoved = Entries.Remove(ChunkId) > 0;
		if (bRemoved)
		{
			TRACE_COUNTER_DECREMENT(ZenJournalSectionedEntries);
			bDirty = true;
		}
	}

	void FJournalChunkInfoPage::InvalidateAll()
	{
		bDirty = Entries.Num() > 0;
		Entries.Empty();
	}

	void FJournalChunkInfoPage::IterateChunkIds(TFunctionRef<void(const FIoChunkId& ChunkId, const FCacheChunkInfo& ChunkInfo)> Callback)
	{
		for (const TTuple<FIoChunkId, FCacheChunkInfo>& Entry : Entries)
		{
			Callback(Entry.Key, Entry.Value);
		}
	}

	FArchive& operator<<(FArchive& Ar, FJournalChunkInfoPage& Page)
	{
		Ar << Page.Entries;
		return Ar;
	}

	FJournalChunkPage::FJournalChunkPage(int64 InPageSize, int32 InMaxEntries)
		: FJournalPageBase(EPageType::Chunk, InPageSize)
		, MaxEntryCount(InMaxEntries)
	{
	}

	EJournalPageResult FJournalChunkPage::Add(const FIoChunkId& ChunkId, const FCacheEntry& Entry)
	{
		if (Entries.Num() >= MaxEntryCount)
		{
			return EJournalPageResult::PageFull;
		}

		FCacheEntry Out;
		if (GetEntry(ChunkId, Entry.ChunkOffset, Entry.ChunkSize, Out))
		{
			return EJournalPageResult::EntryAlreadyExists;
		}

		FJournalStoreEntry StoreEntry;
		StoreEntry.ChunkId = ChunkId;
		StoreEntry.Entry = Entry;
		StoreEntry.bValid = true;
		int32 Idx = Entries.Add(StoreEntry);
		FIntervalMap& IntervalMap = ChunkMap.FindOrAdd(ChunkId);
		IntervalMap.Add(Entry.GetChunkInterval(), Idx);

		TRACE_COUNTER_INCREMENT(ZenJournalSectionedChunks);
		bDirty = true;
		return EJournalPageResult::Ok;
	}

	bool FJournalChunkPage::GetEntry(const FIoChunkId& ChunkId, const uint64 ChunkOffset, const uint64 ChunkSize, FCacheEntry& OutEntry)
	{
		FIntervalMap* IntervalMap = ChunkMap.Find(ChunkId);
		if (IntervalMap)
		{
			int32* EntryIdx = IntervalMap->Find(TInterval<uint64>(ChunkOffset, ChunkSize));
			if (EntryIdx)
			{
				OutEntry = Entries[*EntryIdx].Entry;
				return true;
			}
		}

		return false;
	}

	void FJournalChunkPage::IterateCacheEntriesForChunkId(const FIoChunkId& ChunkId, TFunctionRef<void(const FCacheEntry& Entry)> Callback)
	{
		FIntervalMap* IntervalMap = ChunkMap.Find(ChunkId);
		if (IntervalMap)
		{
			for (auto It = IntervalMap->CreateConstIterator(); It; ++It)
			{
				Callback(Entries[It.Value()].Entry);
			}
		}
	}

	void FJournalChunkPage::IterateCacheEntries(TFunctionRef<void(const FIoChunkId& ChunkId, const FCacheEntry& Entry)> Callback)
	{
		for (FJournalStoreEntry& Entry : Entries)
		{
			Callback(Entry.ChunkId, Entry.Entry);
		}
	}

	void FJournalChunkPage::Invalidate(const FIoChunkId& ChunkId)
	{
		FIntervalMap* IntervalMap = ChunkMap.Find(ChunkId);
		if (!IntervalMap)
		{
			return;
		}

		for (auto It = IntervalMap->CreateIterator(); It; ++It)
		{
			Entries[It.Value()].bValid = false;
		}

		TRACE_COUNTER_DECREMENT(ZenJournalSectionedChunks);
		ChunkMap.Remove(ChunkId);
		bDirty = true;
		bContainsInvalidEntries = true;
	}

	void FJournalChunkPage::InvalidateAll()
	{
		Entries.Empty();
		ChunkMap.Empty();
		bDirty = true;
	}

	FArchive& operator << (FArchive& Ar, FJournalChunkPage& Page)
	{
		bool bNeedsMapRebuild = Ar.IsLoading();
		if (Page.bContainsInvalidEntries)
		{
			TArray<FJournalStoreEntry> ValidEntries;
			ValidEntries.Reserve(Page.Entries.Num());

			const int32 EntriesCount = Page.Entries.Num();
			for (int32 Idx = 0; Idx < EntriesCount; ++Idx)
			{
				if (Page.Entries[Idx].bValid)
				{
					ValidEntries.Add(Page.Entries[Idx]);
				}
			}

			Page.Entries = MoveTemp(ValidEntries);
			Page.bContainsInvalidEntries = false;

			bNeedsMapRebuild = true;
		}

		Ar << Page.Entries;

		if (bNeedsMapRebuild)
		{
			Page.ChunkMap.Empty();

			const int32 EntryCount = Page.Entries.Num();
			for (int32 Idx = 0; Idx < EntryCount; ++Idx)
			{
				FJournalStoreEntry& StoreEntry = Page.Entries[Idx];
				if (!StoreEntry.bValid)
				{
					Page.bContainsInvalidEntries = true;
					continue;
				}

				FJournalChunkPage::FIntervalMap& IntervalMap = Page.ChunkMap.FindOrAdd(StoreEntry.ChunkId);
				IntervalMap.Add(StoreEntry.Entry.GetChunkInterval(), Idx);
			}
		}

		return Ar;
	}

}

#endif // !UE_BUILD_SHIPPING