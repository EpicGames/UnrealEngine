// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CacheJournal.h"

#if !UE_BUILD_SHIPPING

#include "CoreMinimal.h"
#include "Serialization/MemoryReader.h"
#include "Tasks/Task.h"

namespace StorageServer
{
	enum class EPageType : uint8
	{
		Chunk = 1,
		ChunkInfo = 2
	};

	struct FJournalStoreEntry
	{
		FIoChunkId ChunkId;
		FCacheEntry Entry;
		bool bValid;

		bool operator<(const FJournalStoreEntry& Other) const
		{
			return ChunkId < Other.ChunkId;
		}

		friend FArchive& operator<<(FArchive& Ar, FJournalStoreEntry& StoreEntry)
		{
			Ar << StoreEntry.ChunkId;
			Ar << StoreEntry.Entry;
			Ar << StoreEntry.bValid;
			return Ar;
		}
	};

	enum class EJournalPageResult
	{
		Ok,
		EntryAlreadyExists,
		PageFull,
		EntryNotFound
	};

	struct FJournalHeader
	{
		uint32 Magic;
		uint32 Version;
		uint32 PageCount;

		static constexpr int32 SerializedSize = 32; // actual size is 12 bytes, but we reserve some for future use

		friend FArchive& operator << (FArchive& Ar, FJournalHeader& Header)
		{
			Ar << Header.Magic;
			Ar << Header.Version;
			Ar << Header.PageCount;
			return Ar;
		}
	};

	struct FJournalPageHeader
	{
		uint32 Magic;
		uint32 PageSize; // total size of page in bytes, should be used to calculate offset of next page
		uint32 DataSize; // size of data in current page minus header, can be less than page size if page is not full
		FIoHash DataHash; // hash of data in current page
		EPageType Type;

		static constexpr int32 SerializedSize = 64; // actual size is 33 bytes, but we reserve some for future use

		friend FArchive& operator << (FArchive& Ar, FJournalPageHeader& Desc)
		{
			Ar << Desc.Magic;
			Ar << Desc.PageSize;
			Ar << Desc.DataSize;
			Ar << Desc.DataHash;
			Ar << Desc.Type;
			return Ar;
		}
	};

	class FJournalPageBase
	{
	public:
		FJournalPageBase(EPageType InPageType, int64 InPageSize)
			: FilePos(-1)
			, PageSize(InPageSize)
			, PageType(InPageType)
		{
		}

		virtual ~FJournalPageBase() = default;

		[[nodiscard]] virtual bool Contains(const FIoChunkId&) const = 0;
		[[nodiscard]] virtual bool IsFull() const = 0;

		void SetFilePos(int64 FileCursor)
		{
			FilePos = FileCursor;
		}

		int64 GetFilePos() const
		{
			return FilePos;
		}

		int64 GetPageSize() const
		{
			return PageSize;
		}

		bool Flush(IFileHandle* JournalFile, TArray<uint8>& SerializationBuffer);

		friend FArchive& operator<<(FArchive& Ar, FJournalPageBase& Page)
		{
			Page.Serialize(Ar);
			return Ar;
		}

	protected:
		int64 FilePos;
		const int64 PageSize;
		const EPageType PageType;
		bool bDirty = false;

		virtual void Serialize(FArchive& Ar) = 0;
	};

	class FJournalChunkInfoPage : public FJournalPageBase
	{
	public:
		FJournalChunkInfoPage(int64 InPageSize, int32 InMaxEntries);

		EJournalPageResult Add(const FIoChunkId& ChunkId, const FCacheChunkInfo& Entry);
		bool GetEntry(const FIoChunkId& ChunkId, FCacheChunkInfo& Info);
		virtual bool Contains(const FIoChunkId& ChunkId) const override;
		virtual bool IsFull() const override 
		{
			return Entries.Num() >= MaxEntryCount;
		}
		int32 GetEntryCount() const
		{
			return Entries.Num();
		}

		void Invalidate(const FIoChunkId& ChunkId);
		void InvalidateAll();

		void IterateChunkIds(TFunctionRef<void(const FIoChunkId& ChunkId, const FCacheChunkInfo& ChunkInfo)> Callback);

		friend FArchive& operator<<(FArchive& Ar, FJournalChunkInfoPage& Page);

	private:
		int32 MaxEntryCount;
		TMap<FIoChunkId, FCacheChunkInfo> Entries;

		virtual void Serialize(FArchive& Ar) override
		{
			Ar << *this;
		}
	};

	class FJournalChunkPage : public FJournalPageBase
	{
	public:
		FJournalChunkPage(int64 InPageSize, int32 InMaxEntries);

		EJournalPageResult Add(const FIoChunkId& ChunkId, const FCacheEntry& Entry);
		bool GetEntry(const FIoChunkId& ChunkId, const uint64 ChunkOffset, const uint64 ChunkSize, FCacheEntry& OutEntry);
		virtual bool Contains(const FIoChunkId& ChunkId) const override
		{
			return ChunkMap.Contains(ChunkId);
		}

		void IterateCacheEntriesForChunkId(const FIoChunkId& ChunkId, TFunctionRef<void(const FCacheEntry& Entry)> Callback);
		void IterateCacheEntries(TFunctionRef<void(const FIoChunkId& ChunkId, const FCacheEntry& Entry)> Callback);
		virtual bool IsFull() const override
		{
			return Entries.Num() == MaxEntryCount;
		}
		int32 GetEntryCount() const
		{
			return Entries.Num();
		}

		void Invalidate(const FIoChunkId& ChunkId);
		void InvalidateAll();

		friend FArchive& operator<<(FArchive& Ar, FJournalChunkPage& Page);

	private:
		using FIntervalMap = TMap<TInterval<uint64>, int32>;
		TMap<FIoChunkId, FIntervalMap> ChunkMap;
		TArray<FJournalStoreEntry> Entries;
		int32 MaxEntryCount;
		bool bContainsInvalidEntries = false;

		virtual void Serialize(FArchive& Ar) override
		{
			Ar << *this;
		}
	};

	class FCacheJournalSectioned : public ICacheJournal
	{
	public:

		FCacheJournalSectioned(FStringView JournalPath);
		virtual ~FCacheJournalSectioned() override = default;

		virtual void Flush(bool bImmediate) override;
		virtual void InvalidateAll() override;
		virtual void Invalidate(const FIoChunkId& ChunkId) override;

		virtual bool SetChunkInfo(const FIoChunkId& ChunkId, const TOptional<uint64>& OptModHash, const TOptional<int64>& OptRawSize, const TOptional<int32>& OptRawBlockSize) override;
		virtual bool TryGetChunkInfo(const FIoChunkId& ChunkId, FCacheChunkInfo& OutChunkInfo) override;

		virtual bool AddEntry(const FIoChunkId& ChunkId, const FCacheEntry& Entry) override;
		virtual bool TryGetEntry(const FIoChunkId& ChunkId, const uint64 ChunkOffset, const uint64 ChunkSize, FCacheEntry& OutEntry) override;

		virtual void IterateChunkIds(TFunctionRef<void(const FIoChunkId& ChunkId, const FCacheChunkInfo& ChunkInfo)> Callback) override;
		virtual void IterateCacheEntriesForChunkId(const FIoChunkId& ChunkId, TFunctionRef<void(const FCacheEntry& Entry)> Callback) override;
		virtual void IterateCacheEntries(TFunctionRef<void(const FIoChunkId& ChunkId, const FCacheEntry& Entry)> Callback) override;

	private:
		bool LoadCacheJournal();
		void ValidateJournal();

		FJournalChunkInfoPage* FindOrAddChunkInfoPage(const FIoChunkId& ChunkId);
		FJournalChunkPage* FindOrAddChunkPage(const FIoChunkId& ChunkId);

		FJournalChunkInfoPage* AddChunkInfoPage();
		FJournalChunkPage* AddChunkPage();

		int32 AllocateJournalFilePos(int32 PageSize)
		{
			int32 CurrentJournalPos = NextAvailableJournalPos;
			NextAvailableJournalPos += PageSize;
			return CurrentJournalPos;
		}

		void ResetJournalFilePos()
		{
			NextAvailableJournalPos = FJournalHeader::SerializedSize;
		}

	private:
		FString JournalFilePath;
		TUniquePtr<IFileHandle> JournalFileHandle;
		uint32 NextAvailableJournalPos = FJournalHeader::SerializedSize;
		UE::Tasks::TTask<void> FlushTask;

		bool bPagesModified = false;

		TArray<FJournalChunkPage> ChunkPages;
		TArray<FJournalChunkInfoPage> ChunkInfoPages;
		FJournalChunkPage* CurrentChunkPage = nullptr;

		FCriticalSection CriticalSection;

		void FlushImmediate();
	};

}

#endif // !UE_BUILD_SHIPPING
