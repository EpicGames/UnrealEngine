// Copyright Epic Games, Inc. All Rights Reserved.

#include "CacheJournalSimple.h"

#if !UE_BUILD_SHIPPING

#include "HAL/FileManager.h"
#include "Misc/ScopeLock.h"
#include "ProfilingDebugging/CountersTrace.h"

TRACE_DECLARE_INT_COUNTER(ZenJournalSimpleChunks, TEXT("ZenClient/JournalSimple/Chunks"));
TRACE_DECLARE_INT_COUNTER(ZenJournalSimpleEntries, TEXT("ZenClient/JournalSimple/Entries"));
TRACE_DECLARE_INT_COUNTER(ZenJournalSimpleInvalidated, TEXT("ZenClient/JournalSimple/Invalidated"));

namespace StorageServer
{
	FCacheJournalSimple::FCacheJournalSimple(const TCHAR* InFileName, const uint64 InFlushAtWriteCount)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FCacheJournalSimple::Initialize);

		FileName = InFileName;
		FlushAtWriteCount = InFlushAtWriteCount;
		CurrentWriteCount = 0;
		bDirty = false;
		FArchive* Reader = IFileManager::Get().CreateFileReader(*FileName);
		if (Reader)
		{
			FArchive& Ar = *Reader; // also update writer part!

			Ar << ChunkInfos;
			Ar << ChunkCacheEntries;
			delete Reader;
		}

		for (const TTuple<FIoChunkId, FPerChunkCacheEntries>& ChunkEntriesPair : ChunkCacheEntries)
		{
			TRACE_COUNTER_INCREMENT(ZenJournalSimpleChunks);
			TRACE_COUNTER_ADD(ZenJournalSimpleEntries, ChunkEntriesPair.Value.Num());
		}
	}

	FCacheJournalSimple::~FCacheJournalSimple()
	{
	}

	void FCacheJournalSimple::Flush(bool bImmediate)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FCacheJournalSimple::Flush);

		if (bImmediate)
		{
			FlushImmediate();
		}
		else
		{
			if (!bDirty) // no need to sync this
			{
				return;
			}

			FScopeLock Lock(&FlushTaskLock);

			if (FlushTask.IsCompleted())
			{
				FlushTask = UE::Tasks::Launch(TEXT("CacheJournalFlush"), [this]() {FlushImmediate();});
			}
		}
	}

	void FCacheJournalSimple::InvalidateAll()
	{
		FScopeLock Lock(&DataLock);
		ChunkInfos.Empty();
		ChunkCacheEntries.Empty();
		TRACE_COUNTER_SET(ZenJournalSimpleChunks, 0);
		TRACE_COUNTER_SET(ZenJournalSimpleEntries, 0);
		bDirty = true;
	}

	void FCacheJournalSimple::Invalidate(const FIoChunkId& ChunkId)
	{
		FScopeLock Lock(&DataLock);

		TRACE_COUNTER_INCREMENT(ZenJournalSimpleInvalidated);

		FPerChunkCacheEntries* CacheEntries = ChunkCacheEntries.Find(ChunkId);
		if (CacheEntries)
		{
			TRACE_COUNTER_DECREMENT(ZenJournalSimpleChunks);
			TRACE_COUNTER_SUBTRACT(ZenJournalSimpleEntries, CacheEntries->Num());
		}

		ChunkInfos.Remove(ChunkId);
		ChunkCacheEntries.Remove(ChunkId);
		bDirty = true;
	}

	bool FCacheJournalSimple::SetChunkInfo(const FIoChunkId& ChunkId, const TOptional<uint64>& OptModTag, const TOptional<int64>& OptRawSize, const TOptional<int32>& OptRawBlockSize)
	{
		FScopeLock Lock(&DataLock);
		FCacheChunkInfo* ChunkInfo = ChunkInfos.Find(ChunkId);
		if (ChunkInfo)
		{
			bDirty = true; 
			return ChunkInfo->SetChunkInfo(OptModTag, OptRawSize, OptRawBlockSize);
		}

		FCacheChunkInfo NewChunkInfo = {};
		NewChunkInfo.SetChunkInfo(OptModTag, OptRawSize, OptRawBlockSize);
		ChunkInfos.Add(ChunkId, NewChunkInfo);
		bDirty = true;
		return true;
	}

	bool FCacheJournalSimple::TryGetChunkInfo(const FIoChunkId& ChunkId, FCacheChunkInfo& OutChunkInfo)
	{
		FScopeLock Lock(&DataLock);
		FCacheChunkInfo* ChunkInfo = ChunkInfos.Find(ChunkId);
		if (!ChunkInfo)
		{
			return false;
		}
		OutChunkInfo = *ChunkInfo;
		return true;
	}

	bool FCacheJournalSimple::AddEntry(const FIoChunkId& ChunkId, const FCacheEntry& Entry)
	{
		bool bNeedsFlush = false;

		{
			FScopeLock Lock(&DataLock);

			FPerChunkCacheEntries* CacheEntries = ChunkCacheEntries.Find(ChunkId);
			if (!CacheEntries)
			{
				CacheEntries = &ChunkCacheEntries.Add(ChunkId);
				TRACE_COUNTER_INCREMENT(ZenJournalSimpleChunks);
			}

			const TInterval<uint64> ChunkInterval = Entry.GetChunkInterval();
			if (CacheEntries->Contains(ChunkInterval))
			{
				return false;
			}
		
			CacheEntries->Add(ChunkInterval, Entry);
			TRACE_COUNTER_INCREMENT(ZenJournalSimpleEntries);
			bDirty = true;

			if (CurrentWriteCount++ > FlushAtWriteCount && FlushAtWriteCount > 0)
			{
				bNeedsFlush = true;
				CurrentWriteCount = 0;
			}
		}

		if (bNeedsFlush)
		{
			Flush(false);
		}

		return true;
	}

	bool FCacheJournalSimple::TryGetEntry(const FIoChunkId& ChunkId, const uint64 ChunkOffset, const uint64 ChunkSize, FCacheEntry& OutEntry)
	{
		FScopeLock Lock(&DataLock);

		FPerChunkCacheEntries* CacheEntries = ChunkCacheEntries.Find(ChunkId);
		if (!CacheEntries)
		{
			return false;
		}

		const TInterval<uint64> ChunkInterval(ChunkOffset, ChunkSize);
		FCacheEntry* Entry = CacheEntries->Find(ChunkInterval);

		if (!Entry)
		{
			return false;
		}

		OutEntry = *Entry;
		return true;
	}

	void FCacheJournalSimple::IterateChunkIds(TFunctionRef<void(const FIoChunkId& ChunkId, const FCacheChunkInfo& ChunkInfo)> Callback)
	{
		FScopeLock Lock(&DataLock);

		const FCacheChunkInfo EmptyChunkInfo = {};

		for (const TTuple<FIoChunkId, FPerChunkCacheEntries>& Pair : ChunkCacheEntries)
		{
			const FCacheChunkInfo* ChunkInfo = ChunkInfos.Find(Pair.Key);
			Callback(Pair.Key, ChunkInfo ? *ChunkInfo : EmptyChunkInfo);
		}
	}

	void FCacheJournalSimple::IterateCacheEntriesForChunkId(const FIoChunkId& ChunkId, TFunctionRef<void(const FCacheEntry& Entry)> Callback)
	{
		FScopeLock Lock(&DataLock);
	
		FPerChunkCacheEntries* CacheEntries = ChunkCacheEntries.Find(ChunkId);
		if (!CacheEntries)
		{
			return;
		}

		for (const TTuple<TInterval<uint64>, FCacheEntry>& EntryPair : *CacheEntries)
		{
			Callback(EntryPair.Value);
		}
	}

	void FCacheJournalSimple::IterateCacheEntries(TFunctionRef<void(const FIoChunkId& ChunkId, const FCacheEntry& Entry)> Callback)
	{
		FScopeLock Lock(&DataLock);

		for (const TTuple<FIoChunkId, FPerChunkCacheEntries>& ChunkEntriesPair : ChunkCacheEntries)
		{
			for (const TTuple<TInterval<uint64>, FCacheEntry>& EntryPair : ChunkEntriesPair.Value)
			{
				Callback(ChunkEntriesPair.Key, EntryPair.Value);
			}
		}
	}

	void FCacheJournalSimple::FlushImmediate()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FCacheJournalSimple::FlushImmediate);

		FScopeLock Lock(&DataLock);

		if (!bDirty)
		{
			return;
		}

		// TODO avoid writing whole file every time
		FArchive* Writer = IFileManager::Get().CreateFileWriter(*FileName);
		if (!Writer)
		{
			return;
		}

		FArchive& Ar = *Writer;

		// As we only care about chunkid's that are actually cached there is no need to store all chunk id's
		decltype(ChunkInfos) OnlyCachedChunksInfo;

		for (const TTuple<FIoChunkId, FPerChunkCacheEntries>& Pair : ChunkCacheEntries)
		{
			FCacheChunkInfo* Info = nullptr;
			if ((Info = ChunkInfos.Find(Pair.Key)) != nullptr)
			{
				OnlyCachedChunksInfo.Add(Pair.Key, *Info);
			}
		}

		Ar << OnlyCachedChunksInfo;
		Ar << ChunkCacheEntries;

		Writer->Flush();
		delete Writer;
		bDirty = false;
	}
}

#endif
