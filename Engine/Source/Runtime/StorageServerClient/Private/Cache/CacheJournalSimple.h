// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CacheJournal.h"
#include "Containers/Map.h"
#include "HAL/CriticalSection.h"
#include "Tasks/Task.h"
#include <atomic>

#if !UE_BUILD_SHIPPING

namespace StorageServer
{
	// Cross-platform cache journal implementation, baseline for platforms where platform specific implementation is not available.
	class FCacheJournalSimple : public ICacheJournal
	{
	public:
		FCacheJournalSimple(const TCHAR* FileName, const uint64 FlushAtWriteCount);
		virtual ~FCacheJournalSimple() override;

		virtual void Flush(bool bImmediate) override;

		virtual void InvalidateAll() override;
		virtual void Invalidate(const FIoChunkId& ChunkId) override;

		virtual bool SetChunkInfo(
			const FIoChunkId& ChunkId,
			const TOptional<uint64>& OptModTag,
			const TOptional<int64>& OptRawSize,
			const TOptional<int32>& OptRawBlockSize
		) override;
		virtual bool TryGetChunkInfo(
			const FIoChunkId& ChunkId,
			FCacheChunkInfo& OutChunkInfo
		) override;

		virtual bool AddEntry(
			const FIoChunkId& ChunkId,
			const FCacheEntry& Entry
		) override;
		virtual bool TryGetEntry(
			const FIoChunkId& ChunkId,
			const uint64 ChunkOffset,
			const uint64 ChunkSize,
			FCacheEntry& OutEntry
		) override;

		virtual void IterateChunkIds(
			TFunctionRef<void(const FIoChunkId& ChunkId, const FCacheChunkInfo& ChunkInfo)> Callback
		) override;
		virtual void IterateCacheEntriesForChunkId(
			const FIoChunkId& ChunkId,
			TFunctionRef<void(const FCacheEntry& Entry)> Callback
		) override;
		virtual void IterateCacheEntries(
			TFunctionRef<void(const FIoChunkId& ChunkId, const FCacheEntry& Entry)> Callback
		) override;

	private:
		FCriticalSection DataLock, FlushTaskLock;
		FString FileName;
		UE::Tasks::TTask<void> FlushTask;
		uint64 FlushAtWriteCount;
		uint64 CurrentWriteCount;
		std::atomic<bool> bDirty;

		typedef TMap<TInterval<uint64>, FCacheEntry> FPerChunkCacheEntries;
		TMap<FIoChunkId, FCacheChunkInfo> ChunkInfos;
		TMap<FIoChunkId, FPerChunkCacheEntries> ChunkCacheEntries;

		void FlushImmediate();
	};
}

#endif