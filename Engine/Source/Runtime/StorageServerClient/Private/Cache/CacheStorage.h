// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/IoChunkId.h"
#include "IO/IoBuffer.h"
#include "Containers/Array.h"
#include "Templates/Tuple.h"
#include "Containers/UnrealString.h"
#include "Math/UnrealMathUtility.h"

#if !UE_BUILD_SHIPPING

namespace StorageServer
{
	// Bulk storage for caching, transactions are async, non-atomic, best effort, data might be corrupted.
	class ICacheStorage
	{
	public:
		virtual ~ICacheStorage() = default;

		// returns true if cache was created/truncated this session and all data should be assumed lost
		virtual bool IsNewlyCreatedStorage()
		{
			return bNewlyCreatedStorage;
		}

		virtual void Flush() = 0;

		virtual uint64 GetSize() const = 0;

		virtual void Invalidate(const uint64 Offset, const uint64 Size) = 0;

		virtual FIoBuffer Read(
			const uint64 Offset,
			const uint64 ReadSize,
			TOptional<FIoBuffer> OptDestination
		) = 0;

		virtual void WriteAsync(
			const uint64 Offset,
			const void* Buffer,
			const uint64 WriteSize
		) = 0;

	protected:
		bool bNewlyCreatedStorage = false;

		// Limit cache file size to 2GB to stay below file system limitations
		static constexpr uint64 MaxCacheFileSize = 2ull * 1024 * 1024 * 1024;

		// Generate a list of filenames and sizes to be used for backing storage
		static TArray<TTuple<FString, uint64>> GetBackingFileNames(const TCHAR* FileNamePrefix, const uint64 FileSizeTmp)
		{
			TArray<TTuple<FString, uint64>> BackingFiles;
			BackingFiles.SetNum(FMath::DivideAndRoundUp(FileSizeTmp, MaxCacheFileSize));

			const uint64 SizeOfLastFile = FileSizeTmp - MaxCacheFileSize * (BackingFiles.Num() - 1);

			for (int32 i = 0; i < BackingFiles.Num(); ++i)
			{
				// TODO check if disk size is exhausted?
				const FString FileName = FString::Printf(TEXT("%s%u"), FileNamePrefix, i);
				const uint64 DesiredFileSize = i == BackingFiles.Num() - 1 ? SizeOfLastFile : MaxCacheFileSize;

				BackingFiles[i] = { FileName, DesiredFileSize };
			}

			return BackingFiles;
		}

		static bool GetBackingIntervals(const uint64 Offset, const uint64 Size,
										 uint32& IndexA, uint64& OffsetA, uint64& SizeA,
										 uint32& IndexB, uint64& OffsetB, uint64& SizeB)
		{
			IndexA = FMath::DivideAndRoundDown(Offset, MaxCacheFileSize);
			IndexB = FMath::DivideAndRoundDown(Offset + Size - 1, MaxCacheFileSize);

			// we only support writes spanning maximum two backing storage files
			if (!ensureAlways(IndexA + 1 >= IndexB))
			{
				return false;
			}

			if (IndexA == IndexB)
			{
				OffsetA = Offset - IndexA * MaxCacheFileSize;
				SizeA = Size;
				OffsetB = 0;
				SizeB = 0;
				return true;
			}
			else
			{
				OffsetA = Offset - IndexA * MaxCacheFileSize;
				SizeA = IndexB * MaxCacheFileSize - Offset;
				OffsetB = 0;
				SizeB = Size - SizeA;
				return true;
			}
		}
	};
}

#endif