// Copyright Epic Games, Inc. All Rights Reserved.

#include "CacheStorageBlocking.h"

#if !UE_BUILD_SHIPPING

#include "HAL/PlatformFileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "ProfilingDebugging/CountersTrace.h"

TRACE_DECLARE_MEMORY_COUNTER(FrontBuffer, TEXT("ZenClient/BlockingStorage/FrontBuffer"));
TRACE_DECLARE_MEMORY_COUNTER(BackBuffer, TEXT("ZenClient/BlockingStorage/BackBuffer"));

namespace StorageServer
{
	FCacheStorageBlocking::FCacheStorageBlocking(const TCHAR* FileNamePrefix, const uint64 FileSizeTmp)
	{
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		const TArray<TTuple<FString, uint64>> BackingFileNames = GetBackingFileNames(FileNamePrefix, FileSizeTmp);

		BackingFiles.SetNum(BackingFileNames.Num());
		FileSize = 0;

		for (int32 i = 0; i < BackingFiles.Num(); ++i)
		{
			const FString& FileName = BackingFileNames[i].Key;
			const uint64 DesiredFileSize = BackingFileNames[i].Value;

			// TODO check if disk size is exhausted?
			IFileHandle* BackingFile = PlatformFile.OpenWrite(*FileName, true, true);
			if (!ensureAlwaysMsgf(BackingFile, TEXT("Can't open storage server backing file '%s'"), *FileName))
			{
				return;
			}

			if (BackingFile->Size() != DesiredFileSize)
			{
				BackingFile->Truncate(DesiredFileSize);
				bNewlyCreatedStorage = true;
			}

			FileSize += BackingFile->Size();

			BackingFiles[i] = TUniquePtr<IFileHandle>(BackingFile);
		}
	}

	FCacheStorageBlocking::~FCacheStorageBlocking()
	{
		FlushTask.Wait();
	}

	void FCacheStorageBlocking::Flush()
	{
		// don't block if there is a pending flush already
		Flush(true);
	}

	uint64 FCacheStorageBlocking::GetSize() const
	{
		return FileSize;
	}

	FIoBuffer FCacheStorageBlocking::Read(const uint64 Offset, const uint64 ReadSize, TOptional<FIoBuffer> OptDestination)
	{
		// TODO potential improvement is to inspect in-flight write queues and read data from them if needed
		FIoBuffer Destination = OptDestination.IsSet() && OptDestination->GetSize() >= ReadSize ? OptDestination.GetValue() : FIoBuffer(ReadSize);
		Destination.SetSize(ReadSize);
		BackingReadAt(Offset, Destination.GetData(), ReadSize);
		return Destination;
	}

	void FCacheStorageBlocking::WriteAsync(const uint64 Offset, const void* Buffer, const uint64 WriteSize)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FCacheStorageBlocking::WriteAsync);

		// if buffer is too big it's easier to just store it directly
		if (WriteSize >= Buffers.GetBack()->MaxDataSize / 2)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FCacheStorageBlocking::WriteAsync::SyncWrite);
			BackingWriteAt(Offset, Buffer, WriteSize);
		}
		else
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FCacheStorageBlocking::WriteAsync::Write);
			bool bNeedsFlush = false;
			while (true)
			{
				if (bNeedsFlush)
				{
					// wait for buffer swap
					Flush(false);
				}

				FScopeLock ScopeLock(&BackLock);
				if (Buffers.GetBack()->Write(Offset, Buffer, WriteSize))
				{
					return;
				}
				else
				{
					bNeedsFlush = true;
				}
			}
		}
	}

	void FCacheStorageBlocking::Flush(bool bLazy)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FCacheStorageBlocking::Flush);

		FScopeLock ScopeBackLock(&BackLock);

		if (!FlushTask.IsCompleted())
		{
			if (bLazy)
			{
				return;
			}
			else
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FCacheStorageBlocking::Flush::FlushTaskWait);
				FlushTask.Wait();
			}
		}

		// no data in back buffer, nothing to flush
		if (Buffers.GetBack()->Data.Num() == 0)
		{
			return;
		}

		Buffers.Flip();

		TRACE_COUNTER_SET(FrontBuffer, Buffers.GetFront()->Data.Num());
		TRACE_COUNTER_SET(BackBuffer, Buffers.GetBack()->Data.Num());

		FlushTask = UE::Tasks::Launch(TEXT("CacheStorageFlush"), [this]()
		{
			FScopeLock ScopeFrontLock(&FrontLock);
			FWriteQueue* Front = Buffers.GetFront();

			for (const FWriteOp& Operation : Front->Operations)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FCacheStorageBlocking::Flush::Write);
				BackingWriteAt(Operation.OffsetInFile, Operation.Data, Operation.Size);
				TRACE_COUNTER_SUBTRACT(FrontBuffer, Operation.Size);
			}

			Front->Empty();
		}, LowLevelTasks::ETaskPriority::BackgroundNormal);
	}

	void FCacheStorageBlocking::BackingReadAt(const uint64 Offset, void* Buffer, const uint64 Size)
	{
		uint32 IndexA, IndexB;
		uint64 OffsetA, SizeA, OffsetB, SizeB;
		if (!GetBackingIntervals(Offset, Size, IndexA, OffsetA, SizeA, IndexB, OffsetB, SizeB))
		{
			return;
		}

		if (SizeA > 0)
		{
			BackingFiles[IndexA]->ReadAt(static_cast<uint8*>(Buffer), SizeA, OffsetA);
		}

		if (SizeB > 0)
		{
			BackingFiles[IndexB]->ReadAt(static_cast<uint8*>(Buffer) + SizeA, SizeB, OffsetB);
		}
	}

	void FCacheStorageBlocking::BackingWriteAt(const uint64 Offset, const void* Buffer, const uint64 Size)
	{
		uint32 IndexA, IndexB;
		uint64 OffsetA, SizeA, OffsetB, SizeB;
		if (!GetBackingIntervals(Offset, Size, IndexA, OffsetA, SizeA, IndexB, OffsetB, SizeB))
		{
			return;
		}

		FScopeLock ScopeFileWriteLock(&FileWriteLock);

		if (SizeA > 0)
		{
			BackingFiles[IndexA]->Seek(OffsetA);
			BackingFiles[IndexA]->Write(static_cast<const uint8*>(Buffer), SizeA);
		}

		if (SizeB > 0)
		{
			BackingFiles[IndexB]->Seek(OffsetB);
			BackingFiles[IndexB]->Write(static_cast<const uint8*>(Buffer) + SizeA, SizeB);
		}
	}

	bool FCacheStorageBlocking::FWriteQueue::Write(const uint64 OffsetInFile, const void* Buffer, const uint64 Size)
	{
		if (!CanStore(Size))
		{
			return false;
		}

		TRACE_COUNTER_ADD(BackBuffer, Size);

		uint8* Ptr = Data.GetData() + Data.Num();
		Data.SetNum(Data.Num() + Size);
		FMemory::Memcpy(Ptr, Buffer, Size);

		FWriteOp* Last = Operations.Num() > 0 ? &Operations.Last() : nullptr;
		// Try to coalesce write operation with last operation
		// in hopes that in all linear writes we will only have one write op.
		// Another way would be to sort them later and coalesce on sorted data.
		if (Last != nullptr && Last->OffsetInFile + Last->Size == OffsetInFile)
		{
			Last->Size += Size;
		}
		else
		{
			FWriteOp& New = Operations.AddDefaulted_GetRef();
			New.OffsetInFile = OffsetInFile;
			New.Size = Size;
			New.Data = Ptr;
		}

		return true;
	}
}

#endif
