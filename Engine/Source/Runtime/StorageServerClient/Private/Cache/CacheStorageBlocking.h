// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CacheStorage.h"
#include "Tasks/Task.h"

#if !UE_BUILD_SHIPPING

namespace StorageServer
{
	// Cache storage implemented via blocking file operations
	class FCacheStorageBlocking : public ICacheStorage
	{
	public:
		FCacheStorageBlocking(const TCHAR* FileName, const uint64 FileSize);
		virtual ~FCacheStorageBlocking() override;

		virtual void Flush() override;

		virtual uint64 GetSize() const override;

		virtual void Invalidate(const uint64 Offset, const uint64 Size) override {} // no-op

		virtual FIoBuffer Read(
			const uint64 Offset,
			const uint64 ReadSize,
			TOptional<FIoBuffer> OptDestination
		) override;

		virtual void WriteAsync(
			const uint64 Offset,
			const void* Buffer,
			const uint64 WriteSize
		) override;

	private:
		struct FWriteOp
		{
			uint64 OffsetInFile;
			uint64 Size;
			uint8* Data;
		};

		struct FWriteQueue
		{
			static constexpr uint32 MaxOpCount  =  1 * 1024;
			static constexpr uint32 MaxDataSize = 16 * 1024 * 1024;

			TArray<FWriteOp, TInlineAllocator<MaxOpCount * sizeof(FWriteOp)>> Operations;
			TArray<uint8, TInlineAllocator<MaxDataSize>> Data;

			bool CanStore(const uint64 Size) const
			{
				return Data.Num() + Size <= MaxDataSize;
			}
			bool Write(const uint64 OffsetInFile, const void* Buffer, const uint64 Size);
			void Empty()
			{
				Operations.Empty();
				Data.Empty();
			}
		};

		class FDoubleBuffer
		{
		public:
			FDoubleBuffer()
				: Front(&Queue1)
				, Back(&Queue2)
			{}

			FWriteQueue* GetFront() const {return Front;}
			FWriteQueue* GetBack() const {return Back;}

			void Flip()
			{
				if (Front == &Queue1)
				{
					Front = &Queue2;
					Back = &Queue1;
				}
				else
				{
					Front = &Queue1;
					Back = &Queue2;
				}
			}

		private:
			FWriteQueue Queue1;
			FWriteQueue Queue2;
			FWriteQueue* Front;
			FWriteQueue* Back;
		};

		UE::Tasks::TTask<void> FlushTask;
		TArray<TUniquePtr<IFileHandle>> BackingFiles;
		FDoubleBuffer Buffers;
		FCriticalSection FrontLock, BackLock, FileWriteLock;
		uint64 FileSize;

		void Flush(bool bLazy);
		void BackingReadAt(const uint64 Offset, void* Buffer, const uint64 Size);
		void BackingWriteAt(const uint64 Offset, const void* Buffer, const uint64 Size);
	};
}

#endif