// Copyright Epic Games, Inc. All Rights Reserved.

#include "CacheStorageMmap.h"

#if !UE_BUILD_SHIPPING

#include "HAL/PlatformFileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"

namespace StorageServer
{
	FCacheStorageMmap::FCacheStorageMmap(const TCHAR* FileNamePrefix, const uint64 FileSizeTmp)
	{
		if constexpr (!FPlatformProperties::SupportsMemoryMappedFiles())
		{
			ensure(FPlatformProperties::SupportsMemoryMappedFiles());
		}
		else
		{
			const TArray<TTuple<FString, uint64>> BackingFileNames = GetBackingFileNames(FileNamePrefix, FileSizeTmp);

			BackingFiles.SetNum(BackingFileNames.Num());
			TotalSize = 0;

			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

			for (int32 i = 0; i < BackingFiles.Num(); ++i)
			{
				const FString& FileName = BackingFileNames[i].Key;
				const uint64 DesiredFileSize = BackingFileNames[i].Value;
				FBackingFile& BackingFile = BackingFiles[i];

				IFileHandle* PlainFileHandle = PlatformFile.OpenWrite(*FileName, true, true);
				if (!ensureAlwaysMsgf(PlainFileHandle, TEXT("FCacheStorageMmap failed to open cache storage file '%s'"), *FileName))
				{
					return;
				}

				if (PlainFileHandle->Size() != DesiredFileSize)
				{
					// TODO add truncate to IMappedFileHandle?
					PlainFileHandle->Truncate(DesiredFileSize);
					bNewlyCreatedStorage = true;
				}

				const uint64 ActualFileSize = PlainFileHandle->Size();
				delete PlainFileHandle;

				FOpenMappedResult Result = PlatformFile.OpenMappedEx(*FileName, IPlatformFile::EOpenReadFlags::AllowWrite);
				if (!ensureAlwaysMsgf(!Result.HasError(), TEXT("FCacheStorageMmap OpenMappedEx failed to open '%s' due to '%s'"), *FileName, *Result.GetError().GetMessage()))
				{
					return;
				}

				BackingFile.FileHandle = Result.StealValue();
				BackingFile.FileRegion = TUniquePtr<IMappedFileRegion>(BackingFile.FileHandle->MapRegion(0, ActualFileSize, EMappedFileFlags::EFileWritable));
				BackingFile.MapPtr = (uint8*)BackingFile.FileRegion->GetMappedPtr(); // TODO add rw ptr to IMappedFileRegion
				BackingFile.MapSize = BackingFile.FileRegion->GetMappedSize();

				TotalSize += BackingFile.MapSize;
			}
		}
	}

	FCacheStorageMmap::~FCacheStorageMmap()
	{
		BackingFiles.Reset();
		TotalSize = 0;
	}

	void FCacheStorageMmap::Flush()
	{
	}

	void FCacheStorageMmap::Invalidate(const uint64 Offset, const uint64 Size)
	{
	}

	FIoBuffer FCacheStorageMmap::Read(const uint64 Offset, const uint64 ReadSize, TOptional<FIoBuffer> OptDestination)
	{
		if (!IsValidRange(Offset, ReadSize))
		{
			return FIoBuffer();
		}

		uint32 IndexA, IndexB;
		uint64 OffsetA, SizeA, OffsetB, SizeB;
		if (!GetBackingIntervals(Offset, ReadSize, IndexA, OffsetA, SizeA, IndexB, OffsetB, SizeB))
		{
			return FIoBuffer();
		}

		if (SizeA > 0 && SizeB == 0)
		{
			if (OptDestination.IsSet() && OptDestination->GetSize() >= SizeA)
			{
				OptDestination->SetSize(SizeA);
				FMemory::Memcpy(OptDestination->GetData(), BackingFiles[IndexA].MapPtr + OffsetA, SizeA);
				return OptDestination.GetValue();
			}
			else
			{
				return FIoBuffer(FIoBuffer::Wrap, BackingFiles[IndexA].MapPtr + OffsetA, SizeA);
			}
		}

		if (SizeA > 0 && SizeB > 0)
		{
			FIoBuffer Result = (OptDestination.IsSet() && OptDestination->GetSize() >= SizeA + SizeB) ? OptDestination.GetValue() : FIoBuffer(SizeA + SizeB);
			Result.SetSize(SizeA + SizeB);
			FMemory::Memcpy(Result.GetData(), BackingFiles[IndexA].MapPtr + OffsetA, SizeA);
			FMemory::Memcpy(Result.GetData() + SizeA, BackingFiles[IndexB].MapPtr + OffsetB, SizeB);
			return Result;
		}

		return FIoBuffer();
	}

	void FCacheStorageMmap::WriteAsync(const uint64 Offset, const void* Buffer, const uint64 WriteSize)
	{
		if (!IsValidRange(Offset, WriteSize))
		{
			return;
		}

		uint32 IndexA, IndexB;
		uint64 OffsetA, SizeA, OffsetB, SizeB;
		if (!GetBackingIntervals(Offset, WriteSize, IndexA, OffsetA, SizeA, IndexB, OffsetB, SizeB))
		{
			return;
		}

		if (SizeA > 0 && SizeB == 0)
		{
			FMemory::Memcpy(BackingFiles[IndexA].MapPtr + OffsetA, Buffer, SizeA);
		}

		if (SizeA > 0 && SizeB > 0)
		{
			FMemory::Memcpy(BackingFiles[IndexA].MapPtr + OffsetA, Buffer, SizeA);
			FMemory::Memcpy(BackingFiles[IndexB].MapPtr + OffsetB, (const uint8*)Buffer + SizeA, SizeB);
		}
	}
}

#endif

