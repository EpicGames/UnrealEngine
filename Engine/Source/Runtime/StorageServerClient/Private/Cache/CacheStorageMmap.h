// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CacheStorage.h"

#if !UE_BUILD_SHIPPING

#include "Async/MappedFileHandle.h"

namespace StorageServer
{
	// Cache storage implemented via mmap-ing a file on platforms that support mmap
	class FCacheStorageMmap : public ICacheStorage
	{
	public:
		FCacheStorageMmap(const TCHAR* FileNamePrefix, const uint64 FileSize);
		virtual ~FCacheStorageMmap() override;

		virtual void Flush() override;

		virtual uint64 GetSize() const override {return TotalSize;}

		virtual void Invalidate(const uint64 Offset, const uint64 Size) override;

		virtual FIoBuffer Read(
			const uint64 Offset,
			const uint64 ReadSize,
			TOptional<FIoBuffer> OptDestination
		) override;

		virtual void WriteAsync(
			const uint64 Offset,
			const void* Buffer,
			const uint64 WriteSize
		) override;

	private:
		struct FBackingFile
		{
			TUniquePtr<IMappedFileHandle> FileHandle;
			TUniquePtr<IMappedFileRegion> FileRegion;
			uint8* MapPtr;
			uint64 MapSize;

			FBackingFile() = default;
			FBackingFile(FBackingFile&&) = default;
			FBackingFile(const FBackingFile&) = delete;
			FBackingFile& operator=(const FBackingFile&) = delete;
			~FBackingFile()
			{
				// release region before file handle
				FileRegion.Reset();
				FileHandle.Reset();
				MapPtr = nullptr;
				MapSize = 0;
			}
		};

		TArray<FBackingFile> BackingFiles;
		uint64 TotalSize;

		bool IsValidRange(const uint64 Offset, const uint64 Size) const
		{
			return Offset + Size <= TotalSize;
		}
	};
}

#endif