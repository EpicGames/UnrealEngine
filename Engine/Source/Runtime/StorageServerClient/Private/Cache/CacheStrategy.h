// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CacheJournal.h"
#include "CacheStorage.h"
#include "Templates/UniquePtr.h"
#include "Containers/Ticker.h"

#if !UE_BUILD_SHIPPING

namespace StorageServer
{
	class ICacheStrategy
	{
	public:
		ICacheStrategy(TUniquePtr<ICacheJournal>&& InJournal, TUniquePtr<ICacheStorage>&& InStorage, const float FlushInterval)
			: Journal(MoveTemp(InJournal))
			, Storage(MoveTemp(InStorage))
		{
			ensure(Journal.IsValid());
			ensure(Storage.IsValid());

			if (FlushInterval > 0.0f)
			{
				FlushTicker = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &ICacheStrategy::FlushTick), FlushInterval);
			}
			else
			{
				FlushTicker.Reset();
			}
		}

		virtual ~ICacheStrategy()
		{
			if (FlushTicker.IsValid())
			{
				FTSTicker::RemoveTicker(FlushTicker);
				FlushTicker.Reset();
			}
		}

		virtual void Invalidate(const FIoChunkId& ChunkId) = 0;

		virtual void CacheChunkSize(const FIoChunkId& ChunkId, const int64 RawSize)
		{
			Journal->SetChunkInfo(ChunkId, TOptional<uint64>(), RawSize, TOptional<int32>());
		}

		virtual bool TryGetChunkSize(const FIoChunkId& ChunkId, int64& OutRawSize)
		{
			FCacheChunkInfo ChunkInfo;
			if (Journal->TryGetChunkInfo(ChunkId, ChunkInfo))
			{
				if (ChunkInfo.RawSize.IsSet())
				{
					OutRawSize = *ChunkInfo.RawSize;
					return true;
				}
				else
				{
					return false;
				}
			}
			else
			{
				return false;
			}
		}

		virtual bool ReadChunk(
			const FIoChunkId& RequestChunkId,
			const uint64 RequestOffset,
			const uint64 RequestSize,
			TOptional<FIoBuffer> OptDestination,
			FIoBuffer& OutBuffer,
			EStorageServerContentType& OutContentType
		) = 0;

		// TODO cache individual chunk blocks.
		// In case if ResultContentType is CompressedBinary the ResultBuffer will contain 1 or more chunk blocks,
		// we can cache them individually with corresponding block offset and block size,
		// such as if a follow-up ReadChunk request comes in that maps to the same block (but not same offset/size that was used in first request),
		// it would still be fulfilled from cache.
		// This requires unpacking of CompressedBinary and reconstructing its header for each individual chunk block.
		virtual void CacheChunk(
			const FIoChunkId& RequestChunkId,
			const uint64 RequestRawOffset,
			const uint64 RequestRawSize,
			const FIoBuffer& ResultBuffer,
			const EStorageServerContentType ResultContentType,
			const uint64 ResultModTag
		) = 0;

		virtual void Flush()
		{
			Journal->Flush(false);
			Storage->Flush();
		}

		virtual void IterateChunkIds(TFunctionRef<void(const FIoChunkId& ChunkId, const FCacheChunkInfo& ChunkInfo)> Callback)
		{
			Journal->IterateChunkIds(Callback);
		}

	protected:
		TUniquePtr<ICacheJournal> Journal;
		TUniquePtr<ICacheStorage> Storage;
		FTSTicker::FDelegateHandle FlushTicker;

		bool FlushTick(float DeltaTime)
		{
			Flush();
			return true;
		}
	};
}

#endif