// Copyright Epic Games, Inc. All Rights Reserved.

#include "CacheStrategyLinear.h"

#define ZEN_CACHE_VERBOSE_LOG 0

#if !UE_BUILD_SHIPPING

#include "IO/IoBuffer.h"
#include "Misc/App.h"
#include "ProfilingDebugging/CountersTrace.h"

TRACE_DECLARE_MEMORY_COUNTER(ZenLinearStrategyCacheCurrentSize, TEXT("ZenClient/LinearStrategyCache/CurrentSize"));
TRACE_DECLARE_MEMORY_COUNTER(ZenLinearStrategyCacheInvalidSize, TEXT("ZenClient/LinearStrategyCache/InvalidSize"));

#if ZEN_CACHE_VERBOSE_LOG
DEFINE_LOG_CATEGORY_STATIC(LogCacheStrategyLinear, Log, All);
#endif

namespace StorageServer
{
	FCacheStrategyLinear::FCacheStrategyLinear(TUniquePtr<ICacheJournal>&& InJournal, TUniquePtr<ICacheStorage>&& InStorage, const uint64 InAbandonAtInvalidSize, const float FlushInterval, const bool bInvalidate)
		: ICacheStrategy(MoveTemp(InJournal), MoveTemp(InStorage), FlushInterval)
	{
		uint64 NextFreeOffset = 0;
		uint64 TotalStoredSize = 0;
		Journal->IterateCacheEntries([&NextFreeOffset, &TotalStoredSize](const FIoChunkId&, const FCacheEntry& CacheEntry)
		{
			TotalStoredSize += CacheEntry.StorageSize;
			NextFreeOffset = FMath::Max(NextFreeOffset, CacheEntry.StorageOffset + CacheEntry.StorageSize);
		});

		CurrentSize = NextFreeOffset;
		CurrentInvalidSize = NextFreeOffset - TotalStoredSize;
		AbandonAtInvalidSize = InAbandonAtInvalidSize;
		SetCounters();

#if ZEN_CACHE_VERBOSE_LOG
		UE_LOGF(LogCacheStrategyLinear, Display, "Zen linear cache %llu / %llu / %llu", CurrentSize.load(), CurrentInvalidSize, AbandonAtInvalidSize);
#endif

		if (bInvalidate ||
			Storage->IsNewlyCreatedStorage() ||
			CurrentInvalidSize >= AbandonAtInvalidSize)
		{
			InvalidateAll();
		}
	}

	FCacheStrategyLinear::~FCacheStrategyLinear()
	{
	}

	void FCacheStrategyLinear::InvalidateAll()
	{
#if ZEN_CACHE_VERBOSE_LOG
		UE_LOGF(LogCacheStrategyLinear, Display, "Abandoning zen linear cache");
#endif

		FScopeLock ScopeLock(&JournalLock);

		Journal->InvalidateAll();
		CurrentSize = 0;
		CurrentInvalidSize = 0;
		SetCounters();
	}

	void FCacheStrategyLinear::Invalidate(const FIoChunkId& ChunkId, bool bShouldLockJournal)
	{
		UE::TConditionalScopeLock Lock(JournalLock, bShouldLockJournal);

		if (CurrentSize == 0)
		{
			return;
		}

#if ZEN_CACHE_VERBOSE_LOG
		FCacheChunkInfo ChunkInfo = {};
		Journal->TryGetChunkInfo(ChunkId, ChunkInfo);
		UE_LOGF(LogCacheStrategyLinear, Display, "CachePut Invalidate2 %ls was %llx",
			*LexToString(ChunkId),
			ChunkInfo.ModTag.Get(0));
#endif

		Journal->IterateCacheEntriesForChunkId(ChunkId, [this](const FCacheEntry& CacheEntry)
		{
			CurrentInvalidSize += CacheEntry.StorageSize;
		});

		SetCounters();

		if (CurrentInvalidSize >= AbandonAtInvalidSize)
		{
			InvalidateAll();
		}
		else
		{
			Journal->Invalidate(ChunkId);
		}
	}

	bool FCacheStrategyLinear::ReadChunk(const FIoChunkId& RequestChunkId, const uint64 RequestOffset, const uint64 RequestSize, TOptional<FIoBuffer> OptDestination, FIoBuffer& OutBuffer, EStorageServerContentType& OutContentType)
	{
		FCacheEntry Entry = {};
		if (!Journal->TryGetEntry(RequestChunkId, RequestOffset, RequestSize, Entry))
		{
#if ZEN_CACHE_VERBOSE_LOG
			UE_LOGF(LogCacheStrategyLinear, Display, "CacheMiss Key %ls %llu:%llu",
				*LexToString(RequestChunkId),
				RequestOffset,
				RequestSize);
#endif
			return false;
		}

		FIoBuffer Buffer = Storage->Read(Entry.StorageOffset, Entry.StorageSize, OptDestination);
		if (Buffer.GetSize() != Entry.StorageSize)
		{
#if ZEN_CACHE_VERBOSE_LOG
			UE_LOGF(LogCacheStrategyLinear, Display, "CacheMiss Key %ls %llu:%llu failed read",
				*LexToString(RequestChunkId),
				RequestOffset,
				RequestSize);
#endif
			return false;
		}

		// Compare hashes here because storage doesn't guarantee data consistency
		const FIoHash StorageHash = FIoHash::HashBuffer(Buffer.GetData(), Buffer.GetSize());
		if (Entry.StorageHash != StorageHash)
		{
#if ZEN_CACHE_VERBOSE_LOG
			UE_LOGF(LogCacheStrategyLinear, Display, "CacheMiss Hash %ls %llu:%llu->%llu:%llu %ls != %ls",
				*LexToString(RequestChunkId),
				RequestOffset,
				RequestSize,
				Entry.StorageOffset,
				Entry.StorageSize,
				*LexToString(Entry.StorageHash),
				*LexToString(StorageHash)
			);
#endif
			return false;
		}

#if ZEN_CACHE_VERBOSE_LOG
		UE_LOGF(LogCacheStrategyLinear, Display, "CacheHit %ls %llu:%llu->%llu:%llu %ls",
			*LexToString(RequestChunkId),
			RequestOffset,
			RequestSize,
			Entry.StorageOffset,
			Entry.StorageSize,
			*LexToString(Entry.StorageHash)
		);
#endif

		OutBuffer = Buffer;
		OutContentType = Entry.StorageContentType;
		return true;
	}

	void FCacheStrategyLinear::CacheChunk(const FIoChunkId& RequestChunkId, const uint64 RequestOffset, const uint64 RequestSize, const FIoBuffer& Buffer, const EStorageServerContentType ContentType, const uint64 ResultModTag)
	{
		// TODO add hint to ignore some requests to limit overall write speed

		bool bCommitToStorage = false;
		FCacheEntry Entry = {};

		{
			FScopeLock ScopeLock(&JournalLock);
#if ZEN_CACHE_VERBOSE_LOG
			FCacheChunkInfo ChunkInfoBefore = {};
			Journal->TryGetChunkInfo(RequestChunkId, ChunkInfoBefore);
#endif

			if (!UNLIKELY(Journal->SetChunkInfo(RequestChunkId, ResultModTag, TOptional<int64>(), TOptional<int32>())))
			{
				// Invalidate whole chunk in unlikely case if ModTag's don't match, for example if asset was changed in runtime.
#if ZEN_CACHE_VERBOSE_LOG
				UE_LOGF(LogCacheStrategyLinear, Display, "CachePut Invalidate %ls %llu:%llu was %llx become %llx",
					*LexToString(RequestChunkId),
					RequestOffset,
					RequestSize,
					ChunkInfoBefore.ModTag.Get(0),
					ResultModTag
				);
#endif

				Invalidate(RequestChunkId, false);

				// Still need to set chunk info after invalidation
				const bool ModTagValid = Journal->SetChunkInfo(RequestChunkId, ResultModTag, TOptional<int64>(), TOptional<int32>());
				ensure(ModTagValid);
			}

			if (CurrentSize.load() + Buffer.GetSize() >= Storage->GetSize())
			{
				// no storage left
				return;
			}

			Entry.ChunkOffset = RequestOffset;
			Entry.ChunkSize = RequestSize;
			Entry.StorageOffset = CurrentSize.load();
			Entry.StorageSize = Buffer.GetSize();
			Entry.StorageHash = FIoHash::HashBuffer(Buffer.GetData(), Buffer.GetSize());
			Entry.StorageContentType = ContentType;

			if (Journal->AddEntry(RequestChunkId, Entry))
			{
				CurrentSize += Buffer.GetSize();
				bCommitToStorage = true;
				SetCounters();

#if ZEN_CACHE_VERBOSE_LOG
				UE_LOGF(LogCacheStrategyLinear, Display, "CachePut %ls %llu:%llu->%llu:%llu %ls:%llx",
					*LexToString(RequestChunkId),
					RequestOffset,
					RequestSize,
					Entry.StorageOffset,
					Entry.StorageSize,
					*LexToString(Entry.StorageHash),
					ResultModTag
				);
#endif
			}
		}

		if (bCommitToStorage)
		{
			Storage->WriteAsync(Entry.StorageOffset, Buffer.GetData(), Buffer.GetSize());
		}
	}

	void FCacheStrategyLinear::SetCounters()
	{
		TRACE_COUNTER_SET(ZenLinearStrategyCacheCurrentSize, CurrentSize.load());
		TRACE_COUNTER_SET(ZenLinearStrategyCacheInvalidSize, CurrentInvalidSize);
	}
}

#endif
