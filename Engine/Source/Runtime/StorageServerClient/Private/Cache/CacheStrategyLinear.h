// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CacheStrategy.h"
#include <atomic>

#if !UE_BUILD_SHIPPING

namespace StorageServer
{
	// Use cache storage as arena allocator and store all chunks as they come in. No eviction.
	class FCacheStrategyLinear : public ICacheStrategy
	{
	public:
		FCacheStrategyLinear(TUniquePtr<ICacheJournal>&& InJournal, TUniquePtr<ICacheStorage>&& InStorage, const uint64 AbandonAtInvalidSize, const float FlushInterval, const bool bInvalidate);
		virtual ~FCacheStrategyLinear() override;

		void InvalidateAll();
		virtual void Invalidate(const FIoChunkId& ChunkId) override
		{
			Invalidate(ChunkId, true);
		}

		virtual bool ReadChunk(const FIoChunkId& RequestChunkId, const uint64 RequestOffset, const uint64 RequestSize, TOptional<FIoBuffer> OptDestination, FIoBuffer& OutBuffer, EStorageServerContentType& ContentType) override;

		virtual void CacheChunk(const FIoChunkId& RequestChunkId, const uint64 RequestOffset, const uint64 RequestSize, const FIoBuffer& Buffer, const EStorageServerContentType ContentType, const uint64 ResultModTag) override;

	private:
		// Linear cache is implemented as arena allocator
		std::atomic<uint64> CurrentSize; // Current size of arena allocator
		uint64 CurrentInvalidSize; // Amount of bytes with-in arena allocator that are stale
		uint64 AbandonAtInvalidSize; // Consider cache to be a lost cause at this amount of stale data
		FCriticalSection JournalLock;

		void Invalidate(const FIoChunkId& ChunkId, bool bShouldLockJournal);
		void SetCounters();
	};
}

#endif