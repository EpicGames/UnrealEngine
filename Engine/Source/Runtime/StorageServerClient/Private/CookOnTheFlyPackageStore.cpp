// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookOnTheFlyPackageStore.h"

#if WITH_COTF

#include "HAL/PlatformTime.h"
#include "Containers/ChunkedArray.h"
#include "Serialization/MemoryReader.h"
#include "CookOnTheFlyMessages.h"
#include "CookOnTheFly.h"
#include "Misc/PackageName.h"

FCookOnTheFlyPackageStoreBackend::FCookOnTheFlyPackageStoreBackend(UE::Cook::ICookOnTheFlyServerConnection& InCookOnTheFlyServerConnection)
	: CookOnTheFlyServerConnection(InCookOnTheFlyServerConnection)
{
	// Index zero is invalid
	PackageEntries.Add();
	CookOnTheFlyServerConnection.OnMessage().AddRaw(this, &FCookOnTheFlyPackageStoreBackend::OnCookOnTheFlyMessage);

	GetCookedPackagesFromServer();
	GetAllAvailablePackagesFromServer();

	FPackageName::DoesPackageExistOverride().BindLambda([this](FName PackageName)
	{
		return DoesPackageExist(FPackageId::FromName(PackageName));
	});
}

void FCookOnTheFlyPackageStoreBackend::GetCookedPackagesFromServer()
{
	using namespace UE::Cook;
	using namespace UE::ZenCookOnTheFly::Messaging;

	FCookOnTheFlyRequest Request(ECookOnTheFlyMessage::GetCookedPackages);
	FCookOnTheFlyResponse Response = CookOnTheFlyServerConnection.SendRequest(Request).Get();
	if (Response.IsOk())
	{
		FCompletedPackages GetCookedPackagesResponse = Response.GetBodyAs<FCompletedPackages>();

		UE_LOGF(LogCookOnTheFly, Log, "Got '%d' cooked and '%d' failed packages from server",
			GetCookedPackagesResponse.CookedPackages.Num(), GetCookedPackagesResponse.FailedPackages.Num());

		AddPackages(MoveTemp(GetCookedPackagesResponse.CookedPackages), MoveTemp(GetCookedPackagesResponse.FailedPackages),
			TArray<TPair<FPackageId, FName>>());

		LastServerActivtyTime = FPlatformTime::Seconds();
	}
	else
	{
		UE_LOGF(LogCookOnTheFly, Warning, "Failed to send 'GetCookedPackages' request");
	}
}

void FCookOnTheFlyPackageStoreBackend::GetAllAvailablePackagesFromServer()
{
	using namespace UE::Cook;
	using namespace UE::ZenCookOnTheFly::Messaging;

	FCookOnTheFlyRequest Request(ECookOnTheFlyMessage::GetAllOnDiskPackages);
	FCookOnTheFlyResponse Response = CookOnTheFlyServerConnection.SendRequest(Request).Get();
	if (Response.IsOk())
	{
		FGetAllOnDiskPackagesRequest GetAllOnDiskPackagesResponse = Response.GetBodyAs<FGetAllOnDiskPackagesRequest>();
		UE_LOGF(LogCookOnTheFly, Display, "Received list of %d total on-disk packages from server", GetAllOnDiskPackagesResponse.PackageIds.Num());

		AddOnDiskPackages(GetAllOnDiskPackagesResponse.PackageIds);
	}
	else
	{
		UE_LOGF(LogCookOnTheFly, Warning, "Failed to send 'GetAllOnDiskPackages' request");
	}
}

void FCookOnTheFlyPackageStoreBackend::BeginRead()
{
	EntriesLock.ReadLock();
}

void FCookOnTheFlyPackageStoreBackend::EndRead()
{
	TArray<TPair<FPackageId, FName>> PackageIds = MoveTemp(RequestedPackageIds);
	EntriesLock.ReadUnlock();

	if (PackageIds.Num() > 0)
	{
		SendCookRequest(MoveTemp(PackageIds));
	}
}

bool FCookOnTheFlyPackageStoreBackend::DoesPackageExist(FPackageId PackageId)
{
	FReadScopeLock ReadLock(EntriesLock);
	const FEntryInfo* EntryInfo = PackageIdToEntryInfo.Find(PackageId);
	if (EntryInfo == nullptr)	
	{
		// We didn't encounter a cooked version of this package yet, but it may still exist on the server
		return AllServerOnDiskPackages.Find(PackageId) != nullptr;
	}
	else
	{
		return EntryInfo->Status != EPackageStoreEntryStatus::Missing;
	}
}

EPackageStoreEntryStatus FCookOnTheFlyPackageStoreBackend::GetPackageStoreEntry(FPackageId PackageId,
	FName PackageName, FPackageStoreEntry& OutPackageStoreEntry)
{
	FEntryInfo& EntryInfo = PackageIdToEntryInfo.FindOrAdd(PackageId, FEntryInfo());

	if (EntryInfo.Status == EPackageStoreEntryStatus::Ok)
	{
		check(EntryInfo.EntryIndex != INDEX_NONE);
		return CreatePackageStoreEntry(EntryInfo, OutPackageStoreEntry);
	}
	else if (EntryInfo.Status == EPackageStoreEntryStatus::None)
	{
		EntryInfo.Status = EPackageStoreEntryStatus::Pending;
		RequestedPackageIds.Add(TPair<FPackageId, FName>(PackageId, PackageName));
	}

	return EntryInfo.Status; 
}

void FCookOnTheFlyPackageStoreBackend::SendCookRequest(TArray<TPair<FPackageId, FName>> PackageIdsAndNames)
{
	using namespace UE::Cook;
	using namespace UE::ZenCookOnTheFly::Messaging;

	LastClientActivtyTime = FPlatformTime::Seconds();
	
	TArray<FPackageId> PackageIds;
	PackageIds.Reserve(PackageIdsAndNames.Num());
	for (const TPair<FPackageId, FName>& Pair : PackageIdsAndNames)
	{
		PackageIds.Add(Pair.Key);
	}
	FCookOnTheFlyRequest Request(ECookOnTheFlyMessage::CookPackage);
	Request.SetBodyTo(FCookPackageRequest { PackageIds });

	FCookOnTheFlyResponse Response = CookOnTheFlyServerConnection.SendRequest(Request).Get();
	FCookPackageResponse CookResponse = Response.GetBodyAs<FCookPackageResponse>();

	if (Response.IsOk())
	{
		AddPackages(MoveTemp(CookResponse.CookedPackages), MoveTemp(CookResponse.FailedPackages),
			MoveTemp(PackageIdsAndNames));
	}
	else
	{
		UE_LOGF(LogCookOnTheFly, Warning, "Failed to send cook package(s) request");

		FWriteScopeLock WriteLock(EntriesLock);

		for (const FPackageId& PackageId : PackageIds)
		{
			FEntryInfo& EntryInfo = PackageIdToEntryInfo.FindChecked(PackageId);
			if (EntryInfo.Status == EPackageStoreEntryStatus::Pending)
			{
				EntryInfo.Status = EPackageStoreEntryStatus::Missing;
			}
		}
	}
}

EPackageStoreEntryStatus FCookOnTheFlyPackageStoreBackend::CreatePackageStoreEntry(const FEntryInfo& EntryInfo, FPackageStoreEntry& OutPackageStoreEntry)
{
	if (EntryInfo.Status == EPackageStoreEntryStatus::Ok)
	{
		const FPackageStoreEntryResource& Entry = PackageEntries[EntryInfo.EntryIndex];
		OutPackageStoreEntry.ImportedPackageIds = Entry.ImportedPackageIds;
		return EPackageStoreEntryStatus::Ok;
	}
	else
	{
		return EntryInfo.Status;
	}
}

void FCookOnTheFlyPackageStoreBackend::AddPackages(TArray<FPackageStoreEntryResource> Entries, TArray<FPackageId> FailedPackageIds,
	TArray<TPair<FPackageId, FName>> PackageIdsAndNames)
{
	FWriteScopeLock WriteLock(EntriesLock);
		
	for (FPackageId FailedPackageId : FailedPackageIds)
	{
		FEntryInfo& EntryInfo = PackageIdToEntryInfo.FindOrAdd(FailedPackageId, FEntryInfo());
		if (EntryInfo.Status != EPackageStoreEntryStatus::Missing)
		{
			TPair<FPackageId, FName>* FailedPair = PackageIdsAndNames.FindByPredicate(
				[FailedPackageId](TPair<FPackageId, FName>& Pair) { return Pair.Key == FailedPackageId; });
			FName FailedName;
			if (FailedPair)
			{
				FailedName = FailedPair->Value;
			}
			else
			{
				FailedName = FName("<Unknown>");
			}

			UE_LOGF(LogCookOnTheFly, Warning, "0x%ls [Failed]. Failed to cook package %ls.",
				*LexToString(FailedPackageId), *FailedName.ToString());
		}
		EntryInfo.Status = EPackageStoreEntryStatus::Missing;
		PackageStats.Failed++;
	}

	for (FPackageStoreEntryResource& Entry : Entries)
	{
		const FPackageId PackageId = Entry.GetPackageId();
		FEntryInfo& EntryInfo = PackageIdToEntryInfo.FindOrAdd(PackageId, FEntryInfo());

		EntryInfo.Status = EPackageStoreEntryStatus::Ok;
		if (EntryInfo.EntryIndex == INDEX_NONE)
		{
			EntryInfo.EntryIndex = PackageEntries.Add();
		}
		PackageEntries[EntryInfo.EntryIndex] = MoveTemp(Entry);
		PackageStats.Cooked++;

		UE_LOGF(LogCookOnTheFly, Verbose, "0x%ls '%ls' [OK] (Cooked/Failed='%d/%d')",
			*LexToString(PackageId), *PackageEntries[EntryInfo.EntryIndex].PackageName.ToString(), PackageStats.Cooked.Load(), PackageStats.Failed.Load());
	}
}

void FCookOnTheFlyPackageStoreBackend::AddOnDiskPackages(const TArray<FPackageId>& Packages)
{
	FWriteScopeLock WriteLock(EntriesLock);
	AllServerOnDiskPackages.Reserve(Packages.Num());
	for (const FPackageId& PackageID : Packages)
	{
		AllServerOnDiskPackages.Emplace(PackageID);
	}
}

void FCookOnTheFlyPackageStoreBackend::OnCookOnTheFlyMessage(const UE::Cook::FCookOnTheFlyMessage& Message)
{
	using namespace UE::ZenCookOnTheFly::Messaging;

	switch (Message.GetHeader().MessageType)
	{
		case UE::Cook::ECookOnTheFlyMessage::PackagesCooked:
		{
			FCompletedPackages PackagesCookedMessage = Message.GetBodyAs<FCompletedPackages>();

			UE_LOGF(LogCookOnTheFly, Verbose, "Received 'PackagesCooked' message, Cooked='%d', Failed='%d'",
				PackagesCookedMessage.CookedPackages.Num(),
				PackagesCookedMessage.FailedPackages.Num());

			AddPackages(MoveTemp(PackagesCookedMessage.CookedPackages), MoveTemp(PackagesCookedMessage.FailedPackages),
				TArray<TPair<FPackageId, FName>>());

			if (Context)
			{
				Context->PendingEntriesAdded.Broadcast();
			}

			break;
		}

		default:
			break;
	}

	LastServerActivtyTime = FPlatformTime::Seconds();
}

void FCookOnTheFlyPackageStoreBackend::CheckActivity()
{
	const double TimeSinceLastClientActivity = FPlatformTime::Seconds() - LastClientActivtyTime;
	const double TimeSinceLastServerActivity = FPlatformTime::Seconds() - LastServerActivtyTime;
	const double TimeSinceLastWarning = FPlatformTime::Seconds() - LastWarningTime;

	if (TimeSinceLastClientActivity > MaxInactivityTime &&
		TimeSinceLastServerActivity > MaxInactivityTime &&
		TimeSinceLastWarning > TimeBetweenWarning)
	{
		LastWarningTime = FPlatformTime::Seconds();

		UE_LOGF(LogCookOnTheFly, Log, "No server response in '%.2lf' seconds", TimeSinceLastServerActivity);

		UE_LOGF(LogCookOnTheFly, Log, "=== Pending Packages ===");
		{
			FReadScopeLock ReadLock(EntriesLock);
			for (const auto& KeyValue : PackageIdToEntryInfo)
			{
				const FEntryInfo& EntryInfo = KeyValue.Value;
				if (EntryInfo.Status == EPackageStoreEntryStatus::Pending)
				{
					UE_LOGF(LogCookOnTheFly, Log, "0x%ls", *LexToString(KeyValue.Key));
				}
			}
		}
	}
}

#endif // WITH_COTF
