// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_COTF

#include "Containers/ChunkedArray.h"
#include "Serialization/PackageStore.h"
#include "CookOnTheFly.h"
#include "Misc/ScopeRWLock.h"

class FCookOnTheFlyPackageStoreBackend final
	: public IPackageStoreBackend
{
public:
	struct FEntryInfo
	{
		EPackageStoreEntryStatus Status = EPackageStoreEntryStatus::None;
		int32 EntryIndex = INDEX_NONE;
	};

	struct FPackageStats
	{
		TAtomic<uint32> Cooked{ 0 };
		TAtomic<uint32> Failed{ 0 };
	};

	FCookOnTheFlyPackageStoreBackend(UE::Cook::ICookOnTheFlyServerConnection& InCookOnTheFlyServerConnection);

	virtual void OnMounted(TSharedRef<const FPackageStoreBackendContext> InContext) override
	{
		Context = InContext;
	}

	virtual void BeginRead() override;

	virtual void EndRead() override;

	/**
	 *	Returns true if a cooked or uncooked package exists on-disk on the COTF server
	 *	Note this may not be accurate for generated packages 
	 */
	bool DoesPackageExist(FPackageId PackageId);
	virtual EPackageStoreEntryStatus GetPackageStoreEntry(FPackageId PackageId, FName PackageName,
		FPackageStoreEntry& OutPackageStoreEntry) override;
	
	virtual bool GetPackageRedirectInfo(FPackageId PackageId, FName& OutSourcePackageName, FPackageId& OutRedirectedToPackageId) override
	{
		return false;
	}
	
private:
	void GetCookedPackagesFromServer();
	void GetAllAvailablePackagesFromServer();
	void SendCookRequest(TArray<TPair<FPackageId, FName>> PackageIds);
	EPackageStoreEntryStatus CreatePackageStoreEntry(const FEntryInfo& EntryInfo, FPackageStoreEntry& OutPackageStoreEntry);
	void AddPackages(TArray<FPackageStoreEntryResource> Entries, TArray<FPackageId> FailedPackageIds,
		TArray<TPair<FPackageId, FName>> PackageIdsAndNames);
	void AddOnDiskPackages(const TArray<FPackageId>& Packages);
	void OnCookOnTheFlyMessage(const UE::Cook::FCookOnTheFlyMessage& Message);
	void CheckActivity();

	UE::Cook::ICookOnTheFlyServerConnection& CookOnTheFlyServerConnection;
	TSharedPtr<const FPackageStoreBackendContext> Context;
	FRWLock EntriesLock;
	TMap<FPackageId, FEntryInfo> PackageIdToEntryInfo;
	TChunkedArray<FPackageStoreEntryResource> PackageEntries;
	TArray<TPair<FPackageId, FName>> RequestedPackageIds;
	/**
	 * All packages available on-disk on the COTF server. 
	 * Populated from server asset registry when connection is established. These may be cooked or uncooked
	 */ 
	TSet<FPackageId> AllServerOnDiskPackages;	
	FPackageStats PackageStats;

	const double MaxInactivityTime = 20;
	const double TimeBetweenWarning = 10;
	double LastClientActivtyTime = 0;
	double LastServerActivtyTime = 0;
	double LastWarningTime = 0;
};

#endif // WITH_COTF
