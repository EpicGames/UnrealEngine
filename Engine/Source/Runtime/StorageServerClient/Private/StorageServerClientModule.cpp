// Copyright Epic Games, Inc. All Rights Reserved.

#include "StorageServerClientModule.h"

#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "StorageServerPlatformFile.h"

#if !UE_BUILD_SHIPPING

class FStorageServerClientModule : public IStorageServerClientModule
{
public:
	FStorageServerClientModule() = default;
	~FStorageServerClientModule() = default;

	// ~Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// ~End IModuleInterface

	// ~Begin IPlatformFileModule
	virtual IPlatformFile* GetPlatformFile() override;
	// ~End IPlatformFileModule

	// ~Begin IStorageServerClientModule
	virtual IStorageServerPlatformFile* TryCreateCustomPlatformFile(FStringView StoreDirectory, IPlatformFile* Inner) override;
	virtual bool CreateDetachedPlatformFile(FString Host, uint16 Port, FString Project, FString Platform, FDetachedPlatformFileResult& OutResult) override;
	// ~End IStorageServerClientModule

	IPlatformFile* CachedDefaultPlatformFileInstance = nullptr;
};

void FStorageServerClientModule::StartupModule()
{
}

void FStorageServerClientModule::ShutdownModule()
{
}

IPlatformFile* FStorageServerClientModule::GetPlatformFile()
{
	static TUniquePtr<IPlatformFile> DefaultPlatformFileInstance = MakeUnique<FStorageServerPlatformFile>();
	if (!CachedDefaultPlatformFileInstance)
	{
		CachedDefaultPlatformFileInstance = DefaultPlatformFileInstance.Get();
	}
	return DefaultPlatformFileInstance.Get();
}

IStorageServerPlatformFile* FStorageServerClientModule::TryCreateCustomPlatformFile(FStringView StoreDirectory, IPlatformFile* Inner)
{
	TUniquePtr<FStorageServerPlatformFile> StorageServerPlatformFile = MakeUnique<FStorageServerPlatformFile>();
	StorageServerPlatformFile->SetCustomProjectStorePath(StoreDirectory);
	StorageServerPlatformFile->SetAllowPackageIo(false);
	StorageServerPlatformFile->SetAbortOnConnectionFailure(false);
	const TCHAR* CmdLine = FCommandLine::Get();
	if (StorageServerPlatformFile->ShouldBeUsed(Inner, CmdLine) && StorageServerPlatformFile->Initialize(Inner, CmdLine))
	{
		if (FPaths::IsProjectFilePathSet())
		{
			StorageServerPlatformFile->InitializeAfterProjectFilePath();
		}
		return StorageServerPlatformFile.Release();
	}
	return nullptr;
}

bool FStorageServerClientModule::CreateDetachedPlatformFile(FString Host, uint16 Port, FString Project, FString Platform, FDetachedPlatformFileResult& OutResult)
{
	OutResult = {};

	TUniquePtr<FStorageServerPlatformFile> StorageServerPlatformFile = MakeUnique<FStorageServerPlatformFile>();
	StorageServerPlatformFile->SetAllowPackageIo(false);
	StorageServerPlatformFile->SetAbortOnConnectionFailure(false);

	TStringBuilder<1024> CmdLine;
	CmdLine.Appendf(TEXT("-ZenStoreProject=%s "), *Project);
	CmdLine.Appendf(TEXT("-ZenStorePlatform=%s "), *Platform);
	CmdLine.Appendf(TEXT("-ZenStoreHost=%s "), *Host);
	CmdLine.Appendf(TEXT("-ZenStorePort=%u "), Port);
	CmdLine.Appendf(TEXT("-ZenDisableCache"));

	const TCHAR* CmdLineStr = CmdLine.ToString();
	
	if (StorageServerPlatformFile->ShouldBeUsed(nullptr, CmdLineStr) && StorageServerPlatformFile->Initialize(nullptr, CmdLineStr))
	{
		StorageServerPlatformFile->InitializeAfterProjectFilePath();

		if (StorageServerPlatformFile->IsConnectionValid())
		{
			OutResult.PackageStoreBackend = StorageServerPlatformFile->CreatePackageStoreBackend();
			OutResult.IoDispatcherBackend = StorageServerPlatformFile->CreateIoDispatcherBackend();
			OutResult.PlatformFile = MoveTemp(StorageServerPlatformFile);
			return true;
		}
	}

	return false;
}

IMPLEMENT_MODULE(FStorageServerClientModule, StorageServerClient)

#endif // !UE_BUILD_SHIPPING
