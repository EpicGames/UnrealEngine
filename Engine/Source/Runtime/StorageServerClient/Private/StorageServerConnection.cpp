// Copyright Epic Games, Inc. All Rights Reserved.

#include "StorageServerConnection.h"

#include "IO/IoChunkId.h"
#include "IO/IoDispatcher.h"
#include "IPAddress.h"
#include "Misc/App.h"
#include "Misc/ScopeLock.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/StringBuilder.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Memory/MemoryView.h"
#include "Memory/SharedBuffer.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "SocketSubsystem.h"
#include "Serialization/PackageStore.h"
#include "BuiltInHttpClient/BuiltInHttpClient.h"
#include "BuiltInHttpClient/BuiltInHttpClientFSocket.h"
#include "BuiltInHttpClient/BuiltInHttpClientPlatformSocket.h"
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS
#include "BuiltInHttpClient/BuiltInHttpClientUDSSocket.h"
#endif
#include "HAL/PlatformMath.h"
#include "HAL/PlatformProcess.h"
#include "HAL/RunnableThread.h"
#include "HAL/Event.h"

#include "Cache/CacheJournalSimple.h"
#include "Cache/CacheJournalSectioned.h"
#include "Cache/CacheStorageBlocking.h"
#include "Cache/CacheStorageMmap.h"
#include "Cache/CacheStrategyLinear.h"

#if !UE_BUILD_SHIPPING

#ifndef STORAGE_SERVER_PLATFORM_CACHE_SIZE_KB
	#define STORAGE_SERVER_PLATFORM_CACHE_SIZE_KB 4 * 1024 * 1024 // 4 GB by default
#endif

#ifndef STORAGE_SERVER_PLATFORM_CACHE_ABANDON_SIZE_KB
	#define STORAGE_SERVER_PLATFORM_CACHE_ABANDON_SIZE_KB 512 * 1024 // 512 MB by default
#endif

DEFINE_LOG_CATEGORY(LogStorageServerConnection);

TRACE_DECLARE_INT_COUNTER(ZenHttpClientSerializedBytes, TEXT("ZenClient/SerializedBytes (compressed)"));
TRACE_DECLARE_INT_COUNTER(ZenHttpClientThroughputBytes, TEXT("ZenClient/ThroughputBytes (decompressed)"));

TRACE_DECLARE_FLOAT_COUNTER (ZenCacheRatio,     TEXT("ZenClient/Cache/HitRatio"));
TRACE_DECLARE_INT_COUNTER   (ZenCacheHit,       TEXT("ZenClient/Cache/Hit"));
TRACE_DECLARE_INT_COUNTER   (ZenCacheMiss,      TEXT("ZenClient/Cache/Miss"));
TRACE_DECLARE_MEMORY_COUNTER(ZenCacheHitBytes,  TEXT("ZenClient/Cache/HitBytes"));
TRACE_DECLARE_MEMORY_COUNTER(ZenCacheMissBytes, TEXT("ZenClient/Cache/MissBytes"));

static void SetZenCacheRatio()
{
	const int64 HitCount = TRACE_COUNTER_GET(ZenCacheHit);
	const int64 MissCount = TRACE_COUNTER_GET(ZenCacheMiss);
	const float Ratio = (HitCount + MissCount > 0) ? ((float)HitCount / (HitCount + MissCount)) : 0.0f;
	TRACE_COUNTER_SET(ZenCacheRatio, Ratio * 100.0f);
}

static void ZenCacheHit(const uint64 Bytes)
{
	TRACE_COUNTER_ADD(ZenCacheHit, 1);
	TRACE_COUNTER_ADD(ZenCacheHitBytes, Bytes);
	SetZenCacheRatio();
}

static void ZenCacheMiss(const uint64 Bytes)
{
	TRACE_COUNTER_ADD(ZenCacheMiss, 1);
	TRACE_COUNTER_ADD(ZenCacheMissBytes, Bytes);
	SetZenCacheRatio();
}

static FCbObject ProcessResponse(FIoBuffer IoBuffer, EStorageServerContentType ContentType)
{
	FCbObject ResponseObj;
	if (ContentType == EStorageServerContentType::CbObject)
	{
		FMemoryReaderView Reader(IoBuffer.GetView());
		ResponseObj = LoadCompactBinary(Reader).AsObject();
	}
	else if (ContentType == EStorageServerContentType::CompressedBinary)
	{
		FCompressedBuffer Compressed = FCompressedBuffer::FromCompressed(FSharedBuffer::MakeView(IoBuffer.GetData(), IoBuffer.GetSize()));
		FIoBuffer Decompressed(Compressed.GetRawSize());
		if (FCompressedBufferReader(Compressed).TryDecompressTo(Decompressed.GetMutableView(), 0))
		{
			FBufferReader DecompressedAr(Decompressed.GetData(), Decompressed.GetSize(), false);
			ResponseObj = LoadCompactBinary(DecompressedAr).AsObject();
		}
	}

	return ResponseObj;
}

bool FStorageServerConnection::Initialize(TArrayView<const FString> HostAddresses, const int32 Port, const FAnsiStringView& InBaseURI, TArrayView<const FAnsiString> InPersistentHeaders)
{
	BaseURI = InBaseURI;
	PersistentHeaders = InPersistentHeaders;

	TArray<FString> SortedHostAddresses = SortHostAddressesByLocalSubnet(HostAddresses, Port);
	if (BaseURI.StartsWith("/ws/"))
	{
		bIsUsingZenWorkspace = true;
	}

	for (const FString& HostAddress : SortedHostAddresses)
	{
		HttpClient = CreateHttpClient(HostAddress, Port);
		HttpClient->SetPersistentHeaders(PersistentHeaders);
		CurrentHostAddr = HostAddress;
		if (HandshakeRequest())
		{
			UE_LOGF(LogStorageServerConnection, Display, "Zen store connection established to %ls:%i.", *CurrentHostAddr, Port);
			SetupCacheStrategy();
			return true;
		}
	}

	HttpClient.Reset();
	return false;
}

TIoStatusOr<FStorageServerConnection::Workspaces> FStorageServerConnection::GetWorkspaces()
{
	IStorageServerHttpClient::FResult ResultTuple = HttpClient->RequestSync(BaseURI, EStorageServerContentType::CbObject);
	TIoStatusOr<FIoBuffer> RequestResult = ResultTuple.IoStatus;
	if (RequestResult.IsOk())
	{
		FMemoryReaderView Reader(RequestResult.ValueOrDie().GetView());
		FCbObject ResponseObj = LoadCompactBinary(Reader).AsObject();

		Workspaces Result;
		FCbArrayView WorkspacesArray = ResponseObj["workspaces"].AsArrayView();
		for (FCbFieldView WorkspaceField : WorkspacesArray)
		{
			FCbObjectView WorkspaceObject = WorkspaceField.AsObjectView();

			Workspaces::Workspace Workspace = {
				.Id = *WriteToString<64>(WorkspaceObject["id"].AsObjectId()),
				.Root = FString(WorkspaceObject["root_path"].AsString()),
				.AllowShareCreationFromHttp = WorkspaceObject["allow_share_creation_from_http"].AsBool()
			};

			if (!Workspace.Id.IsEmpty())
			{
				FCbArrayView SharesArray = WorkspaceObject["shares"].AsArrayView();
				for (FCbFieldView ShareField : SharesArray)
				{
					FCbObjectView ShareObject = ShareField.AsObjectView();

					Workspaces::Share Share = {
						.Id = *WriteToString<64>(ShareObject["id"].AsObjectId()),
						.Path = FString(ShareObject["share_path"].AsString()),
						.Alias = FString(ShareObject["alias"].AsString())
					};

					if (!Share.Id.IsEmpty())
					{
						Workspace.Shares.Add(Share);
					}
				}
				Result.Workspaces.Add(Workspace);
			}
		}
		return Result;
	}
	return RequestResult.Status();
}

static void PercentEncodeString(FAnsiStringBuilderBase& EncodedBuilder, const FString& String)
{
	for (FString::ElementType C : String)
	{
		switch (C)
		{
		case '!':
		case '#':
		case '$':
		case '&':
		case '\'':
		case '(':
		case ')':
		case '*':
		case '+':
		case ',':
		case '/':
		case ':':
		case ';':
		case '=':
		case '?':
		case '@':
		case '[':
		case ']':
			EncodedBuilder.Append("%02X", C);
			break;
		default:
			EncodedBuilder.Appendf("%c", C);
		}
	}
}

TIoStatusOr<FString> FStorageServerConnection::CreateShare(const FString& WorkspaceId, const FString& SharePath, const FString& Alias)
{
	TAnsiStringBuilder<256> ResourceBuilder;
	ResourceBuilder.Append(BaseURI);
	ResourceBuilder.Append("/");
	ResourceBuilder.Append(WorkspaceId);
	ResourceBuilder.Append("/");
	ResourceBuilder.Append("000000000000000000000000");
	ResourceBuilder.Append("?share_path=");
	PercentEncodeString(ResourceBuilder, SharePath);
	if (!Alias.IsEmpty())
	{
		ResourceBuilder.Append("&alias=");
		PercentEncodeString(ResourceBuilder, Alias);
	}
	IStorageServerHttpClient::FResult ResultTuple = HttpClient->RequestSync(*ResourceBuilder, EStorageServerContentType::Unknown, "PUT");
	TIoStatusOr<FIoBuffer> Result = ResultTuple.IoStatus;
	if (Result.IsOk())
	{
		FMemoryView TextView(Result.ValueOrDie().GetView());
		return FString::ConstructFromPtrSize(reinterpret_cast<const ANSICHAR*>(TextView.GetData()), TextView.GetSize());
	}
	return Result.Status();
}

void FStorageServerConnection::PackageStoreRequest(TFunctionRef<void(FPackageStoreEntryResource&&)> Callback)
{
	TAnsiStringBuilder<256> ResourceBuilder;
	ResourceBuilder.Append(BaseURI).Append("/entries?fieldfilter=packagestoreentry");

	IStorageServerHttpClient::FResult ResultTuple = HttpClient->RequestSync(*ResourceBuilder, EStorageServerContentType::CompressedBinary);
	TIoStatusOr<FIoBuffer> Result = ResultTuple.IoStatus;
	if (Result.IsOk())
	{
		EStorageServerContentType ContentType = ResultTuple.ContentType;
		FCbObject ResponseObj = ProcessResponse(Result.ValueOrDie(), ContentType);
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(StorageServerPackageStoreRequestParseEntries);
			for (FCbField& OplogEntry : ResponseObj["entries"].AsArray())
			{
				FCbObject OplogObj = OplogEntry.AsObject();
				FPackageStoreEntryResource Entry = FPackageStoreEntryResource::FromCbObject(OplogObj["packagestoreentry"].AsObject());
				Callback(MoveTemp(Entry));
			}
		}
	}
	else
	{
		UE_LOGF(LogStorageServerConnection, Fatal, "Failed to read oplog from storage server. '%ls'", *Result.Status().ToString());
	}
}

void FStorageServerConnection::FileManifestRequest(TFunctionRef<void(FIoChunkId Id, FStringView Path, int64 RawSize)> Callback)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStorageServerConnection::FileManifestRequest);

	TAnsiStringBuilder<256> ResourceBuilder;
	ResourceBuilder.Append(BaseURI).Append("/files?refresh=true&fieldnames=id,clientpath,rawsize");

	IStorageServerHttpClient::FResult ResultTuple = HttpClient->RequestSync(*ResourceBuilder, EStorageServerContentType::CompressedBinary);
	TIoStatusOr<FIoBuffer> Result = ResultTuple.IoStatus;
	if (Result.IsOk())
	{
		EStorageServerContentType ContentType = ResultTuple.ContentType;
		FCbObject ResponseObj = ProcessResponse(Result.ValueOrDie(), ContentType);
		for (FCbField& FileArrayEntry : ResponseObj["files"].AsArray())
		{
			FCbObject Entry = FileArrayEntry.AsObject();
			FCbObjectId Id = Entry["id"].AsObjectId();
			int64 ResponseRawSize = Entry["rawsize"].AsInt64(-1);

			TStringBuilder<128> WidePath;
			WidePath.Append(FUTF8ToTCHAR(Entry["clientpath"].AsString()));

			FIoChunkId ChunkId;
			ChunkId.Set(Id.GetView());

			Callback(ChunkId, WidePath, ResponseRawSize);
		}
	}
	else
	{
		UE_LOGF(LogStorageServerConnection, Fatal, "Failed to read file manifest from storage server. '%ls'", *Result.Status().ToString());
	}
}

void FStorageServerConnection::ChunkInfosRequest(TFunctionRef<void(FIoChunkId Id, FIoHash RawHash, int64 RawSize)> Callback)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStorageServerConnection::ChunkInfosRequest);

	TAnsiStringBuilder<256> ResourceBuilder;
	ResourceBuilder.Append(BaseURI).Append("/chunkinfos?fieldnames=id,rawhash,rawsize");

	IStorageServerHttpClient::FResult ResultTuple = HttpClient->RequestSync(*ResourceBuilder, EStorageServerContentType::CbObject);
	TIoStatusOr<FIoBuffer> Result = ResultTuple.IoStatus;
	if (Result.IsOk())
	{
		FMemoryReaderView Reader(Result.ValueOrDie().GetView());
		FCbObject ResponseObj = LoadCompactBinary(Reader).AsObject();
		for (FCbField& FileArrayEntry : ResponseObj["chunkinfos"].AsArray())
		{
			FCbObject Entry = FileArrayEntry.AsObject();
			FCbObjectId Id = Entry["id"].AsObjectId();
			FIoHash RawHash = Entry["rawhash"].AsHash();
			int64 ResponseRawSize = Entry["rawsize"].AsInt64(-1);

			FIoChunkId ChunkId;
			ChunkId.Set(Id.GetView());

			Callback(ChunkId, RawHash, ResponseRawSize);
		}
	}
	else
	{
		UE_LOGF(LogStorageServerConnection, Fatal, "Failed to read file manifest from storage server. '%ls'", *Result.Status().ToString());
	}
}

int64 FStorageServerConnection::ChunkSizeRequest(const FIoChunkId& ChunkId)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStorageServerConnection::ChunkSizeRequest);

	bool bCacheAvailable = FinalizeSetupCacheStrategy();

	int64 ChunkSize = 0;
	if (bCacheAvailable && CacheStrategy.IsValid() && CacheStrategy->TryGetChunkSize(ChunkId, ChunkSize))
	{
		return ChunkSize;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FStorageServerConnection::ChunkSizeRequest::Http);

	TAnsiStringBuilder<256> ResourceBuilder;
	ResourceBuilder.Append(BaseURI);
	ResourceBuilder << "/" << ChunkId << "/info";

	const double StartTime = FPlatformTime::Seconds();
	IStorageServerHttpClient::FResult ResultTuple = HttpClient->RequestSync(*ResourceBuilder, EStorageServerContentType::CbObject);
	TIoStatusOr<FIoBuffer> Result = ResultTuple.IoStatus;
	if (Result.IsOk())
	{
		const double Duration = FPlatformTime::Seconds() - StartTime;
		AddTimingInstance(Duration, Result.ValueOrDie().GetSize());

		FMemoryReaderView Reader(Result.ValueOrDie().GetView());
		FCbObject ResponseObj = LoadCompactBinary(Reader).AsObject();
		ChunkSize = ResponseObj["size"].AsInt64(0);

		if (CacheStrategy.IsValid())
		{
			CacheStrategy->CacheChunkSize(ChunkId, ChunkSize);
		}

		return ChunkSize;
	}
	else if (Result.Status().GetErrorCode() != EIoErrorCode::NotFound)
	{
		UE_LOGF(LogStorageServerConnection, Fatal, "Failed to get chunk size from storage server. '%ls'", *Result.Status().ToString());
	}

	return -1;
}

TIoStatusOr<FIoBuffer> FStorageServerConnection::ReadChunkRequest(
	const FIoChunkId& ChunkId,
	const uint64 Offset,
	const uint64 Size,
	const TOptional<FIoBuffer> OptDestination,
	const bool bHardwareTargetBuffer
)
{
	// TODO move caching functionality to ReadChunkBatchRequest and remove ReadChunkRequest. 
	TRACE_CPUPROFILER_EVENT_SCOPE(FStorageServerConnection::ReadChunkRequest);

	bool bCacheAvailable = FinalizeSetupCacheStrategy();

	const double StartTime = FPlatformTime::Seconds();

	IStorageServerHttpClient::FResult ResultTuple;
	uint64 ResultModTag = 0;

	bool bWasCached = false;
	FIoBuffer CacheChunkBuffer;
	EStorageServerContentType CacheContentType;
	// TODO is there a way to pass destination directly?
	if (bCacheAvailable && CacheStrategy.IsValid() && CacheStrategy->ReadChunk(ChunkId, Offset, Size, TOptional<FIoBuffer>(), CacheChunkBuffer, CacheContentType))
	{
		bWasCached = true;
		ZenCacheHit(CacheChunkBuffer.GetSize());
		ResultTuple = IStorageServerHttpClient::FResult{
			.IoStatus       = TIoStatusOr<FIoBuffer>(CacheChunkBuffer),
			.ContentType    = CacheContentType,
			.HttpStatusCode = 200
		};
	}
	else
	{
#if HAS_STORAGE_SERVER_RPC_GETCHUNKS_API
		if (bIsUsingZenWorkspace)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FStorageServerConnection::ReadChunkRequest::Http);

			TAnsiStringBuilder<256> ResourceBuilder;
			BuildReadChunkRequestUrl(ResourceBuilder, ChunkId, Offset, Size);

			ResultTuple = HttpClient->RequestSync(*ResourceBuilder);
			ResultModTag = 0; // This endpoint doesn't support ModTag.
		}
		else
		{
			TArray<FChunkBatchRequestEntry> ChunkBatchRequests;
			ChunkBatchRequests.Add(FChunkBatchRequestEntry::DataRequest(ChunkId, Offset, Size));

			FIoStatus ResultStatus = ReadChunkBatchRequest(ChunkBatchRequests, [&](FIoChunkId Id, EStorageServerContentType MimeType, FIoBuffer Data, const TOptional<uint64>& ModTag)
				{
					ensure(ChunkId == Id);
					Data.MakeOwned();
					ResultTuple = IStorageServerHttpClient::FResult{
						.IoStatus       = TIoStatusOr<FIoBuffer>(Data),
						.ContentType    = MimeType,
						.HttpStatusCode = 200
					};
					ResultModTag = ModTag.GetValue(); // ModTag must be present for cache invalidation to work
				});

			if (!ResultStatus.IsOk())
			{
				ResultTuple = IStorageServerHttpClient::FResult{
					.IoStatus       = TIoStatusOr<FIoBuffer>(ResultStatus),
					.ContentType    = EStorageServerContentType::Unknown,
					.HttpStatusCode = 0
				};
			}
		}
#else
		TRACE_CPUPROFILER_EVENT_SCOPE(FStorageServerConnection::ReadChunkRequest::Http);

		TAnsiStringBuilder<256> ResourceBuilder;
		BuildReadChunkRequestUrl(ResourceBuilder, ChunkId, Offset, Size);

		ResultTuple = HttpClient->RequestSync(*ResourceBuilder);
		ResultModTag = 0; // This endpoint doesn't support ModTag.
#endif
	}

	TIoStatusOr<FIoBuffer> ResultBuffer = ReadChunkRequestProcessHttpResult(ResultTuple, Offset, Size, OptDestination, bHardwareTargetBuffer);

	if (ResultBuffer.IsOk())
	{
		if (!bWasCached && CacheStrategy.IsValid())
		{
			ZenCacheMiss(ResultTuple.IoStatus.ValueOrDie().GetSize());
			CacheStrategy->CacheChunk(ChunkId, Offset, Size, ResultTuple.IoStatus.ValueOrDie(), ResultTuple.ContentType, ResultModTag);
		}

		const double Duration = FPlatformTime::Seconds() - StartTime;
		AddTimingInstance(Duration, ResultBuffer.ValueOrDie().GetSize());
	}

	return ResultBuffer;
}

struct FOptionalCbField
{
	static TOptional<FIoHash> AsHash(FCbFieldView FieldView)
	{
		return FieldView.IsHash() ? FieldView.AsHash() : TOptional<FIoHash>();
	}

	static TOptional<uint64> AsUInt64(FCbFieldView FieldView)
	{
		return FieldView.IsInteger() ? FieldView.AsUInt64() : TOptional<uint64>();
	}
};

#if HAS_STORAGE_SERVER_RPC_GETCHUNKS_API
FIoStatus FStorageServerConnection::ReadChunkBatchRequest(const TArray<FChunkBatchRequestEntry>& Chunks, TFunctionRef<void(FIoChunkId Id, EStorageServerContentType MimeType, FIoBuffer Data, const TOptional<uint64>& ModTag)> OnResponse, bool bSkipData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStorageServerConnection::ReadChunkBatchRequest);

	if (Chunks.Num() == 0)
	{
		return FIoStatus::Ok;
	}
	else if (Chunks.Num() > 1)
	{
		// TODO Implement multiple requests to same chunkid in same batch.

		TSet<FIoChunkId> RequestedChunkIds;
		for (const FChunkBatchRequestEntry& Chunk: Chunks)
		{
			if (RequestedChunkIds.Contains(Chunk.ChunkId))
			{
				UE_LOGF(LogStorageServerConnection, Fatal, "Multiple requests to the same chunkid are not supported (%ls is already present)", *LexToString(Chunk.ChunkId));
				return FIoStatus::Unknown;
			}

			RequestedChunkIds.Add(Chunk.ChunkId);
		}
	}

	const double StartTime = FPlatformTime::Seconds();

	FCbWriter Writer;
	Writer.BeginObject();
	Writer << "method" << "getchunks";

	Writer.BeginObject("Request");
	if (bSkipData)
	{
		Writer << "SkipData" << bSkipData;
	}
	Writer.BeginArray("Chunks");

	for (const FChunkBatchRequestEntry& Chunk : Chunks)
	{
		Writer.BeginObject();
		if (!bSkipData)
		{
			Writer << "Offset" << Chunk.Offset;
			Writer << "Size" << Chunk.Size;
		}
		Writer << "Oid" << Chunk.ChunkId;
		if (Chunk.ModTag.IsSet())
		{
			Writer << "ModTag" << *Chunk.ModTag;
		}
		Writer.EndObject();
	}

	Writer.EndArray();
	Writer.EndObject();
	Writer.EndObject();

	TAnsiStringBuilder<256> Uri;
	Uri.Append(BaseURI).Append("/rpc");

	FIoBuffer PostPayload(Writer.GetSaveSize());
	Writer.Save(PostPayload.GetMutableView());

	IStorageServerHttpClient::FResult HttpResult = HttpClient->RequestSync(
		*Uri,
		EStorageServerContentType::CompressedBinary,
		"POST",
		PostPayload,
		EStorageServerContentType::CbObject
	);

	TIoStatusOr<FIoBuffer> Result = HttpResult.IoStatus;

	if (!Result.IsOk())
	{
		return Result.Status();
	}

	const double Duration = FPlatformTime::Seconds() - StartTime;
	AddTimingInstance(Duration, Result.ValueOrDie().GetSize());

	// TODO Replace parsing code with FCbPackage.

	struct ResponseHeader
	{
		uint32 Magic;
		uint32 AttachmentCount;
		uint32 Reserved[2];
	} Header;

	FMemoryReaderView View(Result.ValueOrDie().GetView());
	View.Serialize(&Header, sizeof(Header));

	struct AttachmentInfo
	{
		uint64 PayloadSize;
		uint32 Flags;
		FIoHash Hash;

		enum
		{
			IsCompressed = (1u << 0),
			IsObject = (1u << 1),
		};
	};

	if (Header.Magic != 0xaa77aacc)
	{
		UE_LOGF(LogStorageServerConnection, Fatal, "Read incorrect header magic from server response - expected %x, got %x", 0xaa77aacc, Header.Magic);
		return FIoStatus::Unknown;
	}

	TArray<AttachmentInfo> Info;
	Info.AddUninitialized(Header.AttachmentCount + 1); // One extra for the CbPackage root object.

	TMap<FIoHash, int32> AttachmentHashToIndex;

	for (int32 i = 0; i < Info.Num(); ++i)
	{
		View.Serialize(&Info[i], sizeof(Info[i]));

		AttachmentHashToIndex.Add(Info[i].Hash, i);
	}

	FCbObjectView Root(Result.ValueOrDie().GetData() + View.Tell());

	TArray<uint64> InfoOffsets;
	InfoOffsets.AddUninitialized(Header.AttachmentCount + 1);
	for (int32 i = 0; i < Info.Num(); ++i)
	{
		InfoOffsets[i] = (i == 0) ? View.Tell() : InfoOffsets[i - 1] + Info[i - 1].PayloadSize;
	}

	FCbArrayView ResponseChunks = Root["Chunks"].AsArrayView();

	uint64 ResponseCount = 0;
	for (FCbFieldView Field : ResponseChunks)
	{
		const FCbObjectView Chunk = Field.AsObjectView();
		const FCbObjectId Id = Chunk["Id"].AsObjectId();
		const FIoHash RawHash = Chunk["RawHash"].AsHash();
		const FIoHash FragmentHash = Chunk["FragmentHash"].AsHash();
		const FIoHash Hash = Chunk["Hash"].AsHash();
		const TOptional<uint64> ModTag = FOptionalCbField::AsUInt64(Chunk["ModTag"]);

		FIoChunkId ChunkId;
		ChunkId.Set(Id.GetView());

		if (bSkipData)
		{
			OnResponse(ChunkId, EStorageServerContentType::Unknown, FIoBuffer(), ModTag);
		}
		else
		{
			if (!ensureMsgf(RawHash != FIoHash() || FragmentHash != FIoHash() || Hash != FIoHash(), TEXT("Failed to find hash in chunk info returned from server")))
			{
				continue;
			}

			const FIoHash& AttachmentHash = (Hash != FIoHash() ? Hash : (RawHash != FIoHash() ? RawHash : FragmentHash));

			int32* InfoIndex = AttachmentHashToIndex.Find(AttachmentHash);
			if (!ensureMsgf(InfoIndex, TEXT("Failed to find hash in attachments returned from server")))
			{
				continue;
			}

			const AttachmentInfo& Attachment = Info[*InfoIndex];
			ensureAlways(Attachment.Hash == AttachmentHash);

			EStorageServerContentType MimeType = EStorageServerContentType::CompressedBinary;
			if (Attachment.Flags & AttachmentInfo::IsCompressed)
			{
				MimeType = EStorageServerContentType::CompressedBinary;
			}
			else if (Attachment.Flags & AttachmentInfo::IsObject)
			{
				MimeType = EStorageServerContentType::CbObject;
			}
			else
			{
				MimeType = EStorageServerContentType::Binary;
			}

			FIoBuffer Data(FIoBuffer::Wrap, Result.ValueOrDie().GetData() + InfoOffsets[*InfoIndex], Attachment.PayloadSize);
			OnResponse(ChunkId, MimeType, Data, ModTag);
		}

		ResponseCount++;
	}

	return ResponseCount ? FIoStatus::Ok : EIoErrorCode::NotFound;
}
#endif

void FStorageServerConnection::ReadChunkRequestAsync(
	const FIoChunkId& ChunkId,
	const uint64 Offset,
	const uint64 Size,
	const TOptional<FIoBuffer> OptDestination,
	const bool bHardwareTargetBuffer,
	TFunctionRef<void(TIoStatusOr<FIoBuffer> Data)> OnResponse
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStorageServerConnection::ReadChunkRequestAsync);

	const double StartTime = FPlatformTime::Seconds();

	TAnsiStringBuilder<256> ResourceBuilder;
	BuildReadChunkRequestUrl(ResourceBuilder, ChunkId, Offset, Size);

	// TODO use CacheStrategy

	HttpClient->RequestAsync([this, Offset, Size, OptDestination, bHardwareTargetBuffer, OnResponse, StartTime](IStorageServerHttpClient::FResult HttpResultTuple)
	{
		TIoStatusOr<FIoBuffer> ResultBuffer = ReadChunkRequestProcessHttpResult(HttpResultTuple, Offset, Size, OptDestination, bHardwareTargetBuffer);

		if (ResultBuffer.IsOk())
		{
			const double Duration = FPlatformTime::Seconds() - StartTime;
			AddTimingInstance(Duration, ResultBuffer.ValueOrDie().GetSize());
		}

		OnResponse(ResultBuffer);
	}, *ResourceBuilder);
}

void FStorageServerConnection::GetAndResetStats(IStorageServerPlatformFile::FConnectionStats& OutStats)
{
	OutStats.AccumulatedBytes = AccumulatedBytes.exchange(0, std::memory_order_relaxed);
	OutStats.RequestCount = RequestCount.exchange(0, std::memory_order_relaxed);
	OutStats.MinRequestThroughput = MinRequestThroughput.exchange(DBL_MAX, std::memory_order_relaxed);
	OutStats.MaxRequestThroughput = MaxRequestThroughput.exchange(-DBL_MAX, std::memory_order_relaxed);
}

TArray<FString> FStorageServerConnection::SortHostAddressesByLocalSubnet(TArrayView<const FString> HostAddresses, const int32 Port)
{
	bool bAllArePlatformSocketAddresses = true;
	for (const FString& HostAddress : HostAddresses)
	{
		if (!IsPlatformSocketAddress(HostAddress))
		{
			bAllArePlatformSocketAddresses = false;
			break;
		}
	}

	// return array without sorting if it's 0 or 1 addresses or all of them are platform sockets
	if (HostAddresses.Num() <= 1 || bAllArePlatformSocketAddresses)
	{
		return TArray<FString>(HostAddresses);
	}

	TArray<FString> Result;
	ISocketSubsystem& SocketSubsystem = *ISocketSubsystem::Get();

	// Sorting logic in order:
	// - special platform socket address, see PlatformSocketAddress
	// - on desktop, if it's an IPV6 address loopback (ends with ":1")
	// - on desktop, if it's and IPV4 address loopback (starts with "127.0.0")
	// - host IPV4 subnet matches the client subnet (xxx.xxx.xxx)
	// - remaining addresses
	bool bCanBindAll = false;
	bool bAppendPort = false;
	TSharedPtr<FInternetAddr> localAddr = SocketSubsystem.GetLocalHostAddr(*GLog, bCanBindAll);
	FString localAddrStringSubnet = localAddr->ToString(bAppendPort);

	int32 localLastDotPos = INDEX_NONE;
	if (localAddrStringSubnet.FindLastChar(TEXT('.'), localLastDotPos))
	{
		localAddrStringSubnet = localAddrStringSubnet.LeftChop(localAddrStringSubnet.Len() - localLastDotPos);
	}

	TArray<FString> PlatformSocketAddresses;
	TArray<FString> IPV6Loopback;
	TArray<FString> IPV4Loopback;
	TArray<FString> RegularAddresses;
	TArray<FString> HostnameAddresses;

	for (const FString& HostAddress : HostAddresses)
	{
		if (IsPlatformSocketAddress(HostAddress))
		{
			PlatformSocketAddresses.Push(HostAddress);
			continue;
		}

		if (IsHostnameAddress(HostAddress))
		{
			HostnameAddresses.Push(HostAddress);
			continue;
		}

		TSharedPtr<FInternetAddr> Addr = StringToInternetAddr(HostAddress, Port);

		if (!Addr)
		{
			continue;
		}

		FString tempAddrStringSubnet = Addr->ToString(bAppendPort);

#if PLATFORM_DESKTOP || PLATFORM_ANDROID
		if (Addr->GetProtocolType() == FNetworkProtocolTypes::IPv6)
		{
			if (tempAddrStringSubnet.EndsWith(":1"))
			{
				IPV6Loopback.Push(HostAddress);
				continue;
			}
		}
		else
		{
			if (tempAddrStringSubnet.StartsWith("127.0.0."))
			{
				IPV4Loopback.Push(HostAddress);
				continue;
			}
		}
#elif PLATFORM_IOS
		if (Addr->GetProtocolType() == FNetworkProtocolTypes::IPv4)
		{
			// iOS and Mac have an APIPA address for the ethernet-over-usb connection 
			// between the devices.  If we have an address that matches that pattern
			// supplied by the Mac, we should prefer to use it above other reachable
			// addresses.  Treating it as IPV4 loopback accomplishes that.
			if (tempAddrStringSubnet.StartsWith("169.254."))
			{
				IPV4Loopback.Push(HostAddress);
				continue;
			}
		}
#endif

		int32 LastDotPos = INDEX_NONE;
		if (tempAddrStringSubnet.FindLastChar(TEXT('.'), LastDotPos))
		{
			tempAddrStringSubnet = tempAddrStringSubnet.LeftChop(tempAddrStringSubnet.Len() - LastDotPos);
		}

		if (localAddrStringSubnet.Equals(tempAddrStringSubnet))
		{
			RegularAddresses.Insert(HostAddress, 0);
		}
		else
		{
			RegularAddresses.Push(HostAddress);
		}
	}

	Result.Append(PlatformSocketAddresses);
	Result.Append(IPV6Loopback);
	Result.Append(IPV4Loopback);
	Result.Append(RegularAddresses);
	Result.Append(HostnameAddresses);

	return Result;
}

bool FStorageServerConnection::IsPlatformSocketAddress(const FString Address)
{
	return Address.StartsWith(TEXT("platform://"));
}

bool FStorageServerConnection::IsHostnameAddress(const FString Address)
{
	return Address.StartsWith(TEXT("hostname://"));
}

bool FStorageServerConnection::IsUDSSocketAddress(const FString Address)
{
	return Address.EndsWith(TEXT(".sock"));
}

TUniquePtr<IStorageServerHttpClient> FStorageServerConnection::CreateHttpClient(const FString Address, const int32 Port)
{
	TSharedPtr<FInternetAddr> Addr = StringToInternetAddr(Address, Port);

	// Use Address as Hostname if we can't resolve FInternetAddr
	FString HostName = Addr.IsValid() ? Addr->ToString(false) : Address;

	UE_LOGF(LogStorageServerConnection, Display, "Creating zen store connection to %ls:%i (\"%ls\").", *Address, Port, *HostName);

	TUniquePtr<IBuiltInHttpClientSocketPool> SocketPool;
	if (IsPlatformSocketAddress(Address))
	{
		SocketPool = MakeUnique<FBuiltInHttpClientPlatformSocketPool>(Address); 
	}
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS
	else if (IsUDSSocketAddress(Address))
	{
		SocketPool = MakeUnique<FBuiltInHttpClientUDSSocketPool>(Address);
		// The request will be forwarded by the system that reads from the socket to the right target server
		// Hence the host name in this case doesn't matter. We overwrite it to something basic such as localhost
		// so that the socket path such as 'I:/sockets/zenstore.sock' doesn't mess up the URI generation. 
		HostName = "localhost"; 
	}
#endif
	else
	{
		SocketPool = MakeUnique<FBuiltInHttpClientFSocketPool>(Addr, *ISocketSubsystem::Get());
	}

	return MakeUnique<FBuiltInHttpClient>(MoveTemp(SocketPool), HostName);
}

TSharedPtr<FInternetAddr> FStorageServerConnection::StringToInternetAddr(const FString HostAddr, const int32 Port)
{
	TSharedPtr<FInternetAddr> Result = TSharedPtr<FInternetAddr>();

	ISocketSubsystem& SocketSubsystem = *ISocketSubsystem::Get();

	if (IsPlatformSocketAddress(HostAddr))
	{
		return Result;
	}
	else if (IsUDSSocketAddress(HostAddr))
	{
		return Result;
	}

	// Numeric IPV6 addresses can be enclosed in brackets, and must have the brackets stripped before calling GetAddressFromString
	FString ModifiedHostAddr;
	const FString* EffectiveHostAddr = &HostAddr;
	bool bIsHostname = false;
	if (IsHostnameAddress(HostAddr))
	{
		ModifiedHostAddr = FStringView(HostAddr).RightChop(11); // cut off "hostname://"
		EffectiveHostAddr = &ModifiedHostAddr;
		bIsHostname = true;
	}
	else if (!HostAddr.IsEmpty() && HostAddr[0] == TEXT('[') && HostAddr[HostAddr.Len() - 1] == TEXT(']'))
	{
#if PLATFORM_HAS_BSD_SOCKETS && !PLATFORM_HAS_BSD_IPV6_SOCKETS
		// If the platform doesn't have IPV6 BSD Sockets, then handle an attempt at conversion of loopback addresses, and skip and warn about other addresses
		if (HostAddr == TEXT("[::1]"))
		{
			// Substitute IPV4 loopback for IPV6 loopback
			ModifiedHostAddr = TEXT("127.0.0.1");
		}
		else
		{
			UE_LOGF(LogStorageServerConnection, Warning, "Ignoring storage server host IPV6 address on platform that doesn't support IPV6: %ls", *HostAddr);
			return TSharedPtr<FInternetAddr>();
		}
#else
		ModifiedHostAddr = FStringView(HostAddr).Mid(1, HostAddr.Len() - 2);
#endif
		EffectiveHostAddr = &ModifiedHostAddr;
	}

	if (!bIsHostname)
	{
		Result = SocketSubsystem.GetAddressFromString(*EffectiveHostAddr);
	}
	if (!Result.IsValid() || !Result->IsValid())
	{
		FAddressInfoResult GAIRequest = SocketSubsystem.GetAddressInfo(**EffectiveHostAddr, nullptr, EAddressInfoFlags::Default, NAME_None);
		if (GAIRequest.ReturnCode == SE_NO_ERROR && GAIRequest.Results.Num() > 0)
		{
			Result = GAIRequest.Results[0].Address;
		}
	}

	if (Result.IsValid() && Result->IsValid())
	{
		Result->SetPort(Port);
	}

	return Result;
}

bool FStorageServerConnection::ShouldRetryRequest(const IStorageServerHttpClient::FResult& Result)
{
	// Success doesn't need a retry.
	if (Result.IoStatus.IsOk())
	{
		return false;
	}

	// 4xx responses encode a deterministic client-side failure (e.g. 400 Bad Request,
	// 401 Unauthorized, 404 Not Found). Retrying the same request will produce the same
	// response, so treat the whole 4xx range as terminal and let the caller report it.
	if (Result.HttpStatusCode >= 400 && Result.HttpStatusCode < 500)
	{
		return false;
	}

	// Everything else - transport errors (HttpStatusCode == 0, e.g. send/recv failed
	// before getting a response) and 5xx server errors - is potentially transient. Let
	// the caller retry with backoff.
	return true;
}

bool FStorageServerConnection::HandshakeRequest()
{
	// Handshakes are done with a limited per-attempt connection timeout so that we can
	// find out if the destination is unreachable in a timely manner. Multiple attempts
	// (with bounded backoff) tolerate a server that's briefly busy with a full accept
	// backlog at startup time.
	const float ConnectionTimeoutSeconds = 2.0f;
	constexpr uint32 MaxAttempts = 10;

	for (uint32 Attempt = 0; Attempt < MaxAttempts; Attempt++)
	{
		if (Attempt > 0)
		{
			// 50ms, 100ms, 200ms, 400ms, 500ms, 500ms, ... cap at 500ms. Total worst-case
			// backoff across 10 attempts is ~3.25s.
			const int32 Shift = FMath::Min<int32>((int32)Attempt - 1, 4);
			const float BackoffSeconds = FMath::Min(0.5f, 0.05f * (float)(1 << Shift));
			FPlatformProcess::Sleep(BackoffSeconds);
		}

		IStorageServerHttpClient::FResult ResultTuple = HttpClient->RequestSync(
			BaseURI,
			EStorageServerContentType::Unknown,
			"GET",
			TOptional<FIoBuffer>(),
			EStorageServerContentType::Unknown,
			TOptional<FIoBuffer>(),
			ConnectionTimeoutSeconds,
			false
		);
		TIoStatusOr<FIoBuffer> Result = ResultTuple.IoStatus;
		if (Result.IsOk())
		{
			FMemoryReaderView Reader(Result.ValueOrDie().GetView());
			FCbObject ResponseObj = LoadCompactBinary(Reader).AsObject();
			// we currently don't have any concept of protocol versioning, if
			// we succeed in communicating with the endpoint we're good since
			// any breaking API change would need to be done in a backward
			// compatible manner
			return true;
		}

		// Bail out early on a deterministic non-retryable response (e.g. any 4xx).
		// Burning the remaining attempts on something the server has already definitively
		// said no to is wasted time and noise in the logs.
		if (!ShouldRetryRequest(ResultTuple))
		{
			break;
		}
	}

	return false;
}

void FStorageServerConnection::GetDefaultCacheConfiguration(FCacheConfiguration& OutConfiguration)
{
	const TCHAR* CmdLine = FCommandLine::Get();

	const bool bPlatformSupportsCaching = PLATFORM_SUPPORTS_STORAGE_SERVER_CACHE;
	const bool bPlatformEnablesCachingByDefault = PLATFORM_ENABLES_STORAGE_SERVER_CACHE_BY_DEFAULT;

	OutConfiguration.bEnable = bPlatformSupportsCaching && bPlatformEnablesCachingByDefault;

	// always check it first to ensure we disable cache if cmd arg is provided
	if ((FCString::Strstr(CmdLine, TEXT("-ZenDisableCache")) != nullptr) || UE::IsUsingZenPakFileStreaming())
	{
		OutConfiguration.bEnable = false;
	}
	else if (bPlatformSupportsCaching && (FCString::Strstr(CmdLine, TEXT("-ZenEnableCache")) != nullptr))
	{
		OutConfiguration.bEnable = true;
	}

	if (!OutConfiguration.bEnable)
	{
		return;
	}

	OutConfiguration.bForceInvalidate = FCString::Strstr(CmdLine, TEXT("-ZenInvalidateCache")) != nullptr;
	if (!FParse::Value(CmdLine, TEXT("-ZenCacheSizeKB="), OutConfiguration.CacheSizeKB))
	{
		OutConfiguration.CacheSizeKB = STORAGE_SERVER_PLATFORM_CACHE_SIZE_KB;
	}
	if (!FParse::Value(CmdLine, TEXT("-ZenCacheAbandonSizeKB="), OutConfiguration.AbandonSizeKB))
	{
		OutConfiguration.AbandonSizeKB = STORAGE_SERVER_PLATFORM_CACHE_ABANDON_SIZE_KB;
	}
	OutConfiguration.FlushInterval = 10.0f;
	OutConfiguration.FlushEveryNEntries = 0;

	// use sectioned journal by default
	OutConfiguration.bUseSectionedJournal = FCString::Strstr(CmdLine, TEXT("-ZenUseSimpleJournal")) == nullptr;

	// use mmapp'ed storage if available and not explicitely disabled 
	OutConfiguration.bUseMemoryMappedStorage = FPlatformProperties::SupportsMemoryMappedFiles() && (FCString::Strstr(CmdLine, TEXT("-ZenNoMmappedStorage")) == nullptr);
}

#if !PLATFORM_HAS_CUSTOM_STORAGE_SERVER_CACHE_STRATEGY

void FStorageServerConnection::SetupCacheStrategy()
{
	FCacheConfiguration Configuration = {};
	GetDefaultCacheConfiguration(Configuration);
	
	if (!Configuration.bEnable)
	{
		CacheStrategy.Reset();
		return;
	}

	FString DevStoragePath;
#if defined(STORAGE_SERVER_PLATFORM_CACHE_ROOT_DIR)
	DevStoragePath = STORAGE_SERVER_PLATFORM_CACHE_ROOT_DIR;
#elif PLATFORM_ANDROID
	DevStoragePath = FPlatformMisc::GamePersistentDownloadDir();
#elif PLATFORM_IOS
	DevStoragePath = FPlatformMisc::GetDiscardableCacheDir();
#endif

	const uint64 Size = ((uint64)Configuration.CacheSizeKB) * 1024;
	const float FlushInterval = Configuration.FlushInterval;
	const uint64 FlushEveryNEntries = Configuration.FlushEveryNEntries;
	const uint64 AbandonSize = ((uint64)Configuration.AbandonSizeKB) * 1024;
	const bool bShouldInvalidate = Configuration.bForceInvalidate;

	TUniquePtr<StorageServer::ICacheJournal> Journal;
	if (Configuration.bUseSectionedJournal)
	{
		Journal = MakeUnique<StorageServer::FCacheJournalSectioned>(*(DevStoragePath / TEXT("ZenCacheJournalSectioned.db")));
	}
	else
	{
		Journal = MakeUnique<StorageServer::FCacheJournalSimple>(*(DevStoragePath / TEXT("ZenCache.db")), FlushEveryNEntries);
	}

	TUniquePtr<StorageServer::ICacheStorage> Storage;

	if (Configuration.bUseMemoryMappedStorage)
	{
		Storage = MakeUnique<StorageServer::FCacheStorageMmap>(*(DevStoragePath / TEXT("ZenCache.storage")), Size);
	}
	else
	{
		Storage = MakeUnique<StorageServer::FCacheStorageBlocking>(*(DevStoragePath / TEXT("ZenCache.storage")), Size);
	}

	CacheStrategy = MakeUnique<StorageServer::FCacheStrategyLinear>(MoveTemp(Journal), MoveTemp(Storage), AbandonSize, FlushInterval, bShouldInvalidate);

	AsyncQueryLatestServerChunkInfo = MakeShared<FAsyncQueryLatestServerChunkInfo>(*this);
}
#endif // !PLATFORM_HAS_CUSTOM_STORAGE_SERVER_CACHE_STRATEGY

bool FStorageServerConnection::FinalizeSetupCacheStrategy()
{
	if (!CacheStrategy.IsValid())
	{
		return false;
	}

	// we can read from the cache if the asyncronous initialization has completed
	if (!AsyncQueryLatestServerChunkInfo.IsValid())
	{
		return true;
	}

	// don't try to read from the cache until we've got the latest chunk hashes from the server 
	// this will cause files to be read from the server instead of the cache but all early files are
	// fairly small and it is the only way to guarantee they are up to date
	if (!AsyncQueryLatestServerChunkInfo->IsFinished())
	{
		return false;
	}

	// the server chunk info will have been updated by the async init thread.
	// if there is a platform-specific implementation of SetupCacheStrategy that defers cache creation, the expectation is that it will be handled there
	AsyncQueryLatestServerChunkInfo.Reset();
	return true;
}

FStorageServerConnection::FAsyncQueryLatestServerChunkInfo::FAsyncQueryLatestServerChunkInfo(FStorageServerConnection& InOwner)
	: Owner(InOwner)
	, IsCompleted(FPlatformProcess::GetSynchEventFromPool(true))
{
	if (FRunnableThread::Create( this, TEXT("StorageServerCacheEntriesInit"), 0, EThreadPriority::TPri_Normal) == nullptr)
	{
		IsCompleted->Trigger();
	}
}

FStorageServerConnection::FAsyncQueryLatestServerChunkInfo::~FAsyncQueryLatestServerChunkInfo()
{
	IsCompleted->Wait();
	FPlatformProcess::ReturnSynchEventToPool(IsCompleted);
}

bool FStorageServerConnection::FAsyncQueryLatestServerChunkInfo::IsFinished() const
{
	return IsCompleted->Wait(0);
}

void FStorageServerConnection::FAsyncQueryLatestServerChunkInfo::Wait()
{
	IsCompleted->Wait();
}

uint32 FStorageServerConnection::FAsyncQueryLatestServerChunkInfo::Run()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(StorageServerRefreshCacheEntries);

#if HAS_STORAGE_SERVER_RPC_GETCHUNKS_API
	TArray<FChunkBatchRequestEntry> ChunkBatchRequests;
	Owner.CacheStrategy->IterateChunkIds([&](const FIoChunkId& ChunkId, const StorageServer::FCacheChunkInfo& ChunkInfo)
	{
		// Force a default ModTag value so we invalidate the chunk if we don't have modtag for some reason
		const uint64 ModTag = ChunkInfo.ModTag.Get(0);

		ChunkBatchRequests.Emplace(FChunkBatchRequestEntry::VerifyModTagRequest(ChunkId, ModTag));
	});

	TSet<FIoChunkId> ValidChunkIds;
	ValidChunkIds.Reserve(ChunkBatchRequests.Num());
	Owner.ReadChunkBatchRequest(ChunkBatchRequests, [&](FIoChunkId Id, EStorageServerContentType MimeType, FIoBuffer Data, const TOptional<uint64>& ModTag)
	{
		if (ModTag.IsSet())
		{
			// Chunks rpc endpoint indicate invalid chunks by presence of ModTag in result. 
			TRACE_CPUPROFILER_EVENT_SCOPE(StorageServerRefreshCacheEntries::Invalidate);
			Owner.CacheStrategy->Invalidate(Id);
		}
		else
		{
			ValidChunkIds.Add(Id);
		}
	}, true);

	for (const FChunkBatchRequestEntry& Request: ChunkBatchRequests)
	{
		if (!ValidChunkIds.Contains(Request.ChunkId))
		{
			// Absence of chunkid in response means chunk is not present on the server 
			TRACE_CPUPROFILER_EVENT_SCOPE(StorageServerRefreshCacheEntries::Invalidate);
			Owner.CacheStrategy->Invalidate(Request.ChunkId);
		}
	}
#endif

	IsCompleted->Trigger();
	return 0;
}

void FStorageServerConnection::BuildReadChunkRequestUrl(FAnsiStringBuilderBase& Builder, const FIoChunkId& ChunkId, const uint64 Offset, const uint64 Size)
{
	Builder.Append(BaseURI) << "/" << ChunkId;
	bool HaveQuery = false;
	auto AppendQueryDelimiter = [&]
	{
		if (HaveQuery)
		{
			Builder.Append(ANSITEXTVIEW("&"));
		}
		else
		{
			Builder.Append(ANSITEXTVIEW("?"));
			HaveQuery = true;
		}
	};
	if (Offset)
	{
		AppendQueryDelimiter();
		Builder.Appendf("offset=%" UINT64_FMT, Offset);
	}
	if (Size != ~uint64(0))
	{
		AppendQueryDelimiter();
		Builder.Appendf("size=%" UINT64_FMT, Size);
	}
}

TIoStatusOr<FIoBuffer> FStorageServerConnection::ReadChunkRequestProcessHttpResult(
	IStorageServerHttpClient::FResult ResultTuple,
	const uint64 Offset,
	const uint64 Size,
	const TOptional<FIoBuffer> OptDestination,
	const bool bHardwareTargetBuffer
)
{
	TIoStatusOr<FIoBuffer> Result = ResultTuple.IoStatus;
	EStorageServerContentType MimeType = ResultTuple.ContentType;
	if (!Result.IsOk())
	{
		UE_LOG(LogStorageServerConnection, Warning, TEXT("Failed read chunk from storage server. '%s'  Offset:%" UINT64_FMT "  Size:%" UINT64_FMT), *Result.Status().ToString(), Offset, Size);
		return Result.Status();
	}

	FIoBuffer Buffer = Result.ValueOrDie();
	TRACE_COUNTER_ADD(ZenHttpClientSerializedBytes, Buffer.GetSize());

	if (MimeType == EStorageServerContentType::Binary)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FStorageServerConnection::ReadChunkRequest::Binary);

		if (OptDestination.IsSet())
		{
			ensure(OptDestination->GetSize() >= Buffer.GetSize());

			FIoBuffer Destination = OptDestination.GetValue();
			FMemory::Memcpy(Destination.GetData(), Buffer.GetData(), Buffer.GetSize());
			Destination.SetSize(Buffer.GetSize());
			return Destination;
		}
		else
		{
			Buffer.MakeOwned();
			return Buffer;
		}
	}
	else if (MimeType == EStorageServerContentType::CompressedBinary)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FStorageServerConnection::ReadChunkRequest::CompressedBinary);

		FMemoryReaderView Reader(Buffer.GetView());
		FCompressedBuffer CompressedBuffer = FCompressedBuffer::FromCompressed(FSharedBuffer::MakeView(Buffer.GetData(), Buffer.GetSize()));
		FCompressedBufferReader CompressedBufferReader(CompressedBuffer);
		const uint64 RawSize = CompressedBufferReader.GetRawSize();
		if (RawSize > 0)
		{
			const uint64 CompressedOffset = GetCompressedOffset(CompressedBuffer, Offset);
			const uint64 BytesToReadNonTrimmed = Size > 0 ? FMath::Min(Size, RawSize) : RawSize;
			const uint64 BytesToRead = FMath::Min(BytesToReadNonTrimmed, RawSize - CompressedOffset);

			ensure(!OptDestination.IsSet() || OptDestination->GetSize() >= BytesToRead);

			FIoBuffer OutChunk = OptDestination.IsSet() ? OptDestination.GetValue() : FIoBuffer(BytesToRead);
			OutChunk.SetSize(BytesToRead);

			if (CompressedBufferReader.TryDecompressTo(OutChunk.GetMutableView(), CompressedOffset, bHardwareTargetBuffer ? ECompressedBufferDecompressFlags::IntermediateBuffer : ECompressedBufferDecompressFlags::None))
			{
				return OutChunk;
			}
		}
	}

	return FIoStatus(EIoErrorCode::Unknown);
}

uint64 FStorageServerConnection::GetCompressedOffset(const FCompressedBuffer& Buffer, uint64 RawOffset)
{
	if (RawOffset > 0)
	{
		uint64 BlockSize = 0;
		ECompressedBufferCompressor Compressor;
		ECompressedBufferCompressionLevel CompressionLevel;
		const bool bOk = Buffer.TryGetCompressParameters(Compressor, CompressionLevel, BlockSize);
		check(bOk);

		return BlockSize > 0 ? RawOffset % BlockSize : 0;
	}

	return 0;
}

void FStorageServerConnection::AddTimingInstance(const double Duration, const uint64 Bytes)
{
	if ((Duration >= 0.0))
	{
		double tr = ((double)(Bytes * 8) / Duration) / 1000000.0; //Mbps

		AccumulatedBytes.fetch_add(Bytes, std::memory_order_relaxed);
		RequestCount.fetch_add(1, std::memory_order_relaxed);

		double MinTemp = MinRequestThroughput.load(std::memory_order_relaxed);
		while (!MinRequestThroughput.compare_exchange_weak(MinTemp, FMath::Min(MinTemp, tr), std::memory_order_relaxed))
		{
			MinTemp = MinRequestThroughput.load(std::memory_order_relaxed);
		}

		double MaxTemp = MaxRequestThroughput.load(std::memory_order_relaxed);
		while (!MaxRequestThroughput.compare_exchange_weak(MaxTemp, FMath::Max(MaxTemp, tr), std::memory_order_relaxed))
		{
			MaxTemp = MaxRequestThroughput.load(std::memory_order_relaxed);
		}
	}

	TRACE_COUNTER_ADD(ZenHttpClientThroughputBytes, Bytes);
}

// TODO revive FStorageServerChunkBatchRequest
#if 0

class FStorageServerChunkBatchRequest : private FStorageServerRequest
{
public:
	FStorageServerChunkBatchRequest& AddChunk(const FIoChunkId& ChunkId, int64 Offset, int64 Size);
	bool Issue(TFunctionRef<void(uint32 ChunkCount, uint32* ChunkIndices, uint64* ChunkSizes, FStorageServerResponse& ChunkDataStream)> OnResponse);

private:
	friend FStorageServerConnection;

	FStorageServerChunkBatchRequest(FStorageServerConnection& Owner, FAnsiStringView Resource, FAnsiStringView Hostname);

	FStorageServerConnection& Owner;
	int32 ChunkCountOffset = 0;
};

FStorageServerChunkBatchRequest::FStorageServerChunkBatchRequest(FStorageServerConnection& InOwner, FAnsiStringView Resource, FAnsiStringView Hostname)
	: FStorageServerRequest("POST", Resource, Hostname)
	, Owner(InOwner)
{
	uint32 Magic = 0xAAAA'77AC;
	uint32 ChunkCountPlaceHolder = 0;
	uint32 Reserved1 = 0;
	uint32 Reserved2 = 0;
	*this << Magic;
	ChunkCountOffset = BodyBuffer.Num();
	*this << ChunkCountPlaceHolder << Reserved1 << Reserved2;
}

FStorageServerChunkBatchRequest& FStorageServerChunkBatchRequest::AddChunk(const FIoChunkId& ChunkId, int64 Offset, int64 Size)
{
	uint32* ChunkCount = reinterpret_cast<uint32*>(BodyBuffer.GetData() + ChunkCountOffset);
	*this << const_cast<FIoChunkId&>(ChunkId) << *ChunkCount << Offset << Size;
	++(*ChunkCount);
	return *this;
}

bool FStorageServerChunkBatchRequest::Issue(TFunctionRef<void(uint32 ChunkCount, uint32* ChunkIndices, uint64* ChunkSizes, FStorageServerResponse& ChunkDataStream)> OnResponse)
{
	IStorageConnectionSocket* Socket = Send(Owner);
	if (!Socket)
	{
		UE_LOGF(LogStorageServerConnection, Fatal, "Failed to send chunk batch request to storage server.");
		return false;
	}
	FStorageServerResponse Response(Owner, *Socket);
	if (!Response.IsOk())
	{
		UE_LOGF(LogStorageServerConnection, Fatal, "Failed to read chunk batch from storage server. '%ls'", *Response.GetErrorMessage());
		return false;
	}

	uint32 Magic;
	uint32 ChunkCount;
	uint32 Reserved1;
	uint32 Reserved2;
	Response << Magic;
	if (Magic != 0xbada'b00f)
	{
		UE_LOGF(LogStorageServerConnection, Fatal, "Invalid magic in chunk batch response from storage server.");
		return false;
	}
	Response << ChunkCount;
	if (ChunkCount > INT32_MAX)
	{
		UE_LOGF(LogStorageServerConnection, Fatal, "Invalid chunk count in chunk batch response from storage server.");
		return false;
	}
	Response << Reserved1;
	Response << Reserved2;

	TArray<uint32, TInlineAllocator<64>> ChunkIndices;
	ChunkIndices.Reserve(ChunkCount);
	TArray<uint64, TInlineAllocator<64>> ChunkSizes;
	ChunkSizes.Reserve(ChunkCount);
	for (uint32 Index = 0; Index < ChunkCount; ++Index)
	{
		uint32 ChunkIndex;
		uint32 Flags;
		int64 ChunkSize;
		Response << ChunkIndex;
		Response << Flags;
		Response << ChunkSize;
		ChunkIndices.Add(ChunkIndex);
		ChunkSizes.Emplace(ChunkSize);
	}
	OnResponse(ChunkCount, ChunkIndices.GetData(), ChunkSizes.GetData(), Response);
	Owner.AddTimingInstance(GetDuration(), (double)Response.Tell());
	return true;
}

#endif

#endif
