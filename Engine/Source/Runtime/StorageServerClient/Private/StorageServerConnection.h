// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Compression/CompressedBuffer.h"
#include "Containers/UnrealString.h"
#include "Containers/AnsiString.h"
#include "IO/IoChunkId.h"
#include "Misc/StringBuilder.h"
#include "Memory/MemoryFwd.h"
#include "Templates/SharedPointer.h"
#include "SocketTypes.h"
#include "HAL/PlatformTime.h"
#include "HAL/Runnable.h"
#include "StorageServerHttpClient.h"
#include "IStorageServerPlatformFile.h"
#include "Cache/CacheStrategy.h"

#ifndef PLATFORM_SUPPORTS_STORAGE_SERVER_CACHE
	#define PLATFORM_SUPPORTS_STORAGE_SERVER_CACHE (PLATFORM_WINDOWS || PLATFORM_ANDROID || PLATFORM_IOS)
#endif

#if PLATFORM_SUPPORTS_STORAGE_SERVER_CACHE && PLATFORM_ANDROID
	#define PLATFORM_ENABLES_STORAGE_SERVER_CACHE_BY_DEFAULT 1
#endif

#ifndef PLATFORM_ENABLES_STORAGE_SERVER_CACHE_BY_DEFAULT
	#define PLATFORM_ENABLES_STORAGE_SERVER_CACHE_BY_DEFAULT 0
#endif

#ifndef PLATFORM_HAS_CUSTOM_STORAGE_SERVER_CACHE_STRATEGY
	#define PLATFORM_HAS_CUSTOM_STORAGE_SERVER_CACHE_STRATEGY 0
#endif

#ifndef HAS_STORAGE_SERVER_RPC_GETCHUNKS_API // zen server 5.5.16 introduced a new API to request chunks
	#define HAS_STORAGE_SERVER_RPC_GETCHUNKS_API 1
#endif

#if !UE_BUILD_SHIPPING

DECLARE_LOG_CATEGORY_EXTERN(LogStorageServerConnection, Log, All);

struct FPackageStoreEntryResource;

class FStorageServerConnection
{
public:
	FStorageServerConnection() = default;
	~FStorageServerConnection() = default;

	bool Initialize(TArrayView<const FString> HostAddresses, const int32 Port, const FAnsiStringView& InBaseURI, TArrayView<const FAnsiString> InPersistentHeaders = {});

	struct Workspaces
	{
		struct Share
		{
			FString Id;
			FString Path;
			FString Alias;
		};

		struct Workspace
		{
			FString Id;
			FString Root;
			bool AllowShareCreationFromHttp = false;

			TArray<Share> Shares;
		};
		TArray<Workspace> Workspaces;
	};

	TIoStatusOr<Workspaces> GetWorkspaces();
	TIoStatusOr<FString> CreateShare(const FString& WorkspaceId, const FString& SharePath, const FString& Alias);
	bool IsConnectedToWorkspace() const
	{
		return bIsUsingZenWorkspace;
	}

	void PackageStoreRequest(TFunctionRef<void(FPackageStoreEntryResource&&)> Callback);
	void FileManifestRequest(TFunctionRef<void(FIoChunkId Id, FStringView Path, int64 RawSize)> Callback);
	void ChunkInfosRequest(TFunctionRef<void(FIoChunkId Id, FIoHash RawHash, int64 RawSize)> Callback);
	int64 ChunkSizeRequest(const FIoChunkId& ChunkId);
	TIoStatusOr<FIoBuffer> ReadChunkRequest(
		const FIoChunkId& ChunkId,
		const uint64 Offset,
		const uint64 Size,
		const TOptional<FIoBuffer> OptDestination,
		const bool bHardwareTargetBuffer
	);

	void ReadChunkRequestAsync(
		const FIoChunkId& ChunkId,
		const uint64 Offset,
		const uint64 Size,
		const TOptional<FIoBuffer> OptDestination,
		const bool bHardwareTargetBuffer,
		TFunctionRef<void(TIoStatusOr<FIoBuffer> Data)> OnResponse
	);

#if HAS_STORAGE_SERVER_RPC_GETCHUNKS_API
	// Matches input parameters for ProjectStore::GetChunks in zen server
	struct FChunkBatchRequestEntry {
		FIoChunkId ChunkId;
		uint64 Offset;
		uint64 Size;
		TOptional<uint64> ModTag;

		static FChunkBatchRequestEntry DataRequest(const FIoChunkId& ChunkId, const uint64 Offset, const uint64 Size)
		{
			TOptional<uint64> EmptyModTag;
			return FChunkBatchRequestEntry {ChunkId, Offset, Size, EmptyModTag};
		}

		static FChunkBatchRequestEntry VerifyModTagRequest(const FIoChunkId& ChunkId, const uint64 ModTag)
		{
			return FChunkBatchRequestEntry {ChunkId, 0, (uint64)-1, ModTag};
		}
	};

	FIoStatus ReadChunkBatchRequest(
		const TArray<FChunkBatchRequestEntry>& Chunks,
		TFunctionRef<void(FIoChunkId Id, EStorageServerContentType MimeType, FIoBuffer Data, const TOptional<uint64>& ModTag)> OnResponse,
		bool bSkipData = false // if bSkipData is true, then OnResponse will only be called for requested chunks that have either empty or different modtags
	);
#endif

	FStringView GetHostAddr() const
	{
		return CurrentHostAddr;
	}

	void GetAndResetStats(IStorageServerPlatformFile::FConnectionStats& OutStats);

private:
	TUniquePtr<IStorageServerHttpClient> HttpClient;
	TUniquePtr<StorageServer::ICacheStrategy> CacheStrategy;
	FAnsiString BaseURI;
	FString CurrentHostAddr;
	TArray<FAnsiString> PersistentHeaders;
	bool bIsUsingZenWorkspace = false;		    // is the connection to the /ws/ endpoint

	// Stats
	std::atomic<uint64> AccumulatedBytes = 0;
	std::atomic<uint32> RequestCount = 0;
	std::atomic<double> MinRequestThroughput = DBL_MAX;
	std::atomic<double> MaxRequestThroughput = -DBL_MAX;

	TArray<FString> SortHostAddressesByLocalSubnet(TArrayView<const FString> HostAddresses, const int32 Port);
	static bool IsPlatformSocketAddress(const FString Address);
	static bool IsUDSSocketAddress(const FString Address);
	static bool IsHostnameAddress(const FString Address); 
	TUniquePtr<IStorageServerHttpClient> CreateHttpClient(const FString Address, const int32 Port);
	TSharedPtr<FInternetAddr> StringToInternetAddr(const FString Address, const int32 Port);
	bool HandshakeRequest();

	struct FCacheConfiguration
	{
		bool bEnable = false;					// set to true to enable cache
		bool bForceInvalidate = false;			// invalidate cache if set to true
		int32 CacheSizeKB = 0;					// total size of cache in kb
		float FlushInterval = 0.f;				// interval at which to flush cache in seconds
		int32 FlushEveryNEntries = 0;			// set to >0 to flush journal every N new entries
		int32 AbandonSizeKB = 0;				// set to >0 to abandon cache if amount of invalid data goes over threshold
		bool bUseSectionedJournal = false;		// use the sectioned journal instead of the simple TMap variant
		bool bUseMemoryMappedStorage = false;	// use mmapped cache storage backend
	};
	// Returns true if the given request result represents a transient failure that is
	// worth retrying. The intent is to retry transport-level failures and 5xx server
	// responses while leaving 4xx alone (they encode deterministic client-side errors
	// like "Not Found" or "Bad Request" that an identical retry would only repeat).
	static bool ShouldRetryRequest(const IStorageServerHttpClient::FResult& Result);

	void GetDefaultCacheConfiguration(FCacheConfiguration& OutConfiguration);
	void SetupCacheStrategy();
	bool FinalizeSetupCacheStrategy();
	void BuildReadChunkRequestUrl(FAnsiStringBuilderBase& Builder, const FIoChunkId& ChunkId, const uint64 Offset, const uint64 Size);
	static TIoStatusOr<FIoBuffer> ReadChunkRequestProcessHttpResult(
		IStorageServerHttpClient::FResult ResultTuple,
		const uint64 Offset,
		const uint64 Size,
		const TOptional<FIoBuffer> OptDestination,
		const bool bHardwareTargetBuffer
	);
	static uint64 GetCompressedOffset(const FCompressedBuffer& Buffer, uint64 RawOffset);
	void AddTimingInstance(const double Duration, const uint64 Bytes);

	class FAsyncQueryLatestServerChunkInfo : public FRunnable
	{
	public:
		FAsyncQueryLatestServerChunkInfo(FStorageServerConnection& InOwner);
		virtual ~FAsyncQueryLatestServerChunkInfo();
		
		bool IsFinished() const;
		void Wait();
	private:
		virtual uint32 Run() override;

		FStorageServerConnection& Owner;
		class FEvent* IsCompleted;
	};

	TSharedPtr<FAsyncQueryLatestServerChunkInfo, ESPMode::ThreadSafe> AsyncQueryLatestServerChunkInfo;
};

#endif
