// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IO/IoBuffer.h"
#include "IO/IoStatus.h"
#include "Misc/Optional.h"

#if !UE_BUILD_SHIPPING

enum class EStorageServerContentType : uint8
{
	Unknown = 0,
	CbObject,
	Binary,
	CompressedBinary,
};

class IStorageServerHttpClient
{
public:
	// Result of an HTTP request.
	struct FResult
	{
		// IO status (and on success the response body) for the request. Callers should
		// inspect .IsOk() before dereferencing. Named IoStatus rather than Status to
		// avoid confusion with the HTTP-level HttpStatusCode field below.
		TIoStatusOr<FIoBuffer>    IoStatus;

		// Content-Type reported by the server (Unknown when no response was received
		// or the server omitted the header).
		EStorageServerContentType ContentType    = EStorageServerContentType::Unknown;

		// Raw HTTP response status code (e.g. 200, 404, 503). Zero when no HTTP
		// response was actually received - e.g. the request failed before reaching
		// the server, or the result was synthesized from a local cache hit.
		int32                     HttpStatusCode = 0;
	};

	using FResultCallback = TFunction<void(FResult)>;

	virtual ~IStorageServerHttpClient() = default;

	/** Set headers that will be included with every request. Each entry should be a complete header line (e.g. "Authorization: Basic dWU6cGFzcw=="). */
	virtual void SetPersistentHeaders(TArrayView<const FAnsiString> InHeaders) = 0;

	virtual FResult RequestSync(
		FAnsiStringView Url,
		EStorageServerContentType Accept = EStorageServerContentType::Unknown,
		FAnsiStringView Verb = "GET",
		TOptional<FIoBuffer> OptPayload = TOptional<FIoBuffer>(),
		EStorageServerContentType PayloadContentType = EStorageServerContentType::Unknown,
		TOptional<FIoBuffer> OptDestination = TOptional<FIoBuffer>(),
		float TimeoutSeconds = -1.f,
		const bool bReportErrors = true
	) = 0;

	virtual void RequestAsync(
		FResultCallback&& Callback,
		FAnsiStringView Url,
		EStorageServerContentType Accept = EStorageServerContentType::Unknown,
		FAnsiStringView Verb = "GET",
		TOptional<FIoBuffer> OptPayload = TOptional<FIoBuffer>(),
		EStorageServerContentType PayloadContentType = EStorageServerContentType::Unknown,
		TOptional<FIoBuffer> OptDestination = TOptional<FIoBuffer>(),
		float TimeoutSeconds = -1.f,
		const bool bReportErrors = true
	) = 0;
};

#endif
