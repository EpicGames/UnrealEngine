// Copyright Epic Games, Inc. All Rights Reserved.

#include "StorageServerIoDispatcherBackend.h"
#include "StorageServerConnection.h"
#include "Misc/QueuedThreadPool.h"
#include "Misc/ScopeLock.h"
#include "Misc/CommandLine.h"
#include "HAL/RunnableThread.h"
#include "ProfilingDebugging/IoStoreTrace.h"

#if !UE_BUILD_SHIPPING

int32 GStorageServerIoDispatcherMaxActiveBatchCount = 4;
static FAutoConsoleVariableRef CVar_StorageServerIoDispatcherMaxActiveBatchCount(
	TEXT("s.StorageServerIoDispatcherMaxActiveBatchCount"),
	GStorageServerIoDispatcherMaxActiveBatchCount,
	TEXT("StorageServer IoDispatcher max submitted batches count.")
);

int32 GStorageServerIoDispatcherBatchSize = 16;
static FAutoConsoleVariableRef CVar_StorageServerIoDispatcherBatchSize(
	TEXT("s.StorageServerIoDispatcherBatchSize"),
	GStorageServerIoDispatcherBatchSize,
	TEXT("StorageServer IoDispatcher batch size.")
);

FStorageServerIoDispatcherBackend::FStorageServerIoDispatcherBackend(FStorageServerConnection& InConnection)
	: Connection(InConnection)
	, NewRequestEvent(FPlatformProcess::GetSynchEventFromPool(false))
	, BatchCompletedEvent(FPlatformProcess::GetSynchEventFromPool(false))
{
}

FStorageServerIoDispatcherBackend::~FStorageServerIoDispatcherBackend()
{
	Shutdown();
	FPlatformProcess::ReturnSynchEventToPool(NewRequestEvent);
	FPlatformProcess::ReturnSynchEventToPool(BatchCompletedEvent);
}

void FStorageServerIoDispatcherBackend::Shutdown()
{
	if (Thread)
	{
		delete Thread;
		Thread = nullptr;
	}
}

void FStorageServerIoDispatcherBackend::Initialize(TSharedRef<const FIoDispatcherBackendContext> InContext)
{
	BackendContext = InContext;
	Thread = FRunnableThread::Create(this, TEXT("IoService"), 0, TPri_AboveNormal);
}

void FStorageServerIoDispatcherBackend::Stop()
{
	bStopRequested = true;
	NewRequestEvent->Trigger();
}

uint32 FStorageServerIoDispatcherBackend::Run()
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/StorageServer"));
	const int32 BatchCount = GStorageServerIoDispatcherMaxActiveBatchCount;
	for (int32 BatchIndex = 0; BatchIndex < BatchCount; ++BatchIndex)
	{
		FBatch* Batch = new FBatch(*this);
		Batch->Next = FirstAvailableBatch;
		FirstAvailableBatch = Batch;
	}

	FBatch* CurrentBatch = nullptr;
	while (!bStopRequested)
	{
		for (;;)
		{
			if (!CurrentBatch)
			{
				CurrentBatch = FirstAvailableBatch;
				if (!CurrentBatch)
				{
					WaitForBatchToComplete();
					CurrentBatch = FirstAvailableBatch;
					check(CurrentBatch);
				}
				FirstAvailableBatch = CurrentBatch->Next;
				CurrentBatch->Next = nullptr;
			}

			FIoRequestImpl* Request = RequestQueue.Pop();
			if (!Request)
			{
				break;
			}
			
			check(!Request->NextRequest);
			if (CurrentBatch->RequestsTail)
			{
				CurrentBatch->RequestsTail->NextRequest = Request;
			}
			else
			{
				CurrentBatch->RequestsHead = Request;
			}
			CurrentBatch->RequestsTail = Request;
			++CurrentBatch->RequestsCount;

			if (CurrentBatch->RequestsCount == GStorageServerIoDispatcherBatchSize)
			{
				SubmitBatch(CurrentBatch);
				CurrentBatch = nullptr;
			}
		}
		if (CurrentBatch && CurrentBatch->RequestsCount > 0)
		{
			SubmitBatch(CurrentBatch);
			CurrentBatch = nullptr;
		}
		NewRequestEvent->Wait();
	}

	for (int32 BatchIndex = 0; BatchIndex < BatchCount; ++BatchIndex)
	{
		if (!FirstAvailableBatch)
		{
			if (!WaitForBatchToComplete(10000))
			{
				UE_LOGF(LogIoDispatcher, Warning, "Outstanding requests when shutting down storage server backend");
				return 0;
			}
		}
		check(FirstAvailableBatch);
		FBatch* Batch = FirstAvailableBatch;
		FirstAvailableBatch = FirstAvailableBatch->Next;
		delete Batch;
	}
	return 0;
}

bool FStorageServerIoDispatcherBackend::Resolve(FIoRequestImpl* Request)
{
	check(Request);
	if (BackendContext->bIsMultiThreaded)
	{
		RequestQueue.Push(*Request);
		NewRequestEvent->Trigger();
	}
	else
	{
		TRACE_IOSTORE_BACKEND_REQUEST_STARTED(Request, this);
		TRACE_CPUPROFILER_EVENT_SCOPE(StorageServerIoDispatcherReadChunk);
		const TOptional<FIoBuffer> OptDestination = Request->Options.GetTargetVa() ? FIoBuffer(FIoBuffer::Wrap, Request->Options.GetTargetVa(), Request->Options.GetSize()) : TOptional<FIoBuffer>();
		const bool bHardwareTargetBuffer = EnumHasAnyFlags(Request->Options.GetFlags(), EIoReadOptionsFlags::HardwareTargetBuffer);
		TIoStatusOr<FIoBuffer> Result = Connection.ReadChunkRequest(Request->ChunkId, Request->Options.GetOffset(), Request->Options.GetSize(), OptDestination, bHardwareTargetBuffer); 
		if (Result.IsOk())
		{
			Request->SetResult(Result.ValueOrDie());
			TRACE_IOSTORE_BACKEND_REQUEST_COMPLETED(Request, Result.ValueOrDie().GetSize());
		}
		else
		{
			Request->SetFailed();
			TRACE_IOSTORE_BACKEND_REQUEST_FAILED(Request);
		}
		if (CompletedRequestsTail)
		{
			CompletedRequestsTail->NextRequest = Request;
		}
		else
		{
			CompletedRequestsHead = Request;
		}
		CompletedRequestsTail = Request;
	}
	return true;
}

void FStorageServerIoDispatcherBackend::ResolveIoRequests(FIoRequestList Requests, FIoRequestList& OutUnresolved)
{
	while (FIoRequestImpl* Request = Requests.PopHead())
	{
		if (Resolve(Request) == false)
		{
			OutUnresolved.AddTail(Request);
		}
	}
}

bool FStorageServerIoDispatcherBackend::DoesChunkExist(const FIoChunkId& ChunkId) const
{
	return GetSizeForChunk(ChunkId).IsOk();
}

TIoStatusOr<uint64> FStorageServerIoDispatcherBackend::GetSizeForChunk(const FIoChunkId& ChunkId) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(StorageServerIoDispatcherGetSizeForChunk);
	int64 ChunkSize = Connection.ChunkSizeRequest(ChunkId);
	if (ChunkSize >= 0)
	{
		return ChunkSize;
	}
	else
	{
		return FIoStatus(EIoErrorCode::NotFound);
	}
}

FIoRequestImpl* FStorageServerIoDispatcherBackend::GetCompletedIoRequests()
{
	FScopeLock Lock(&CompletedRequestsCritical);
	FIoRequestImpl* Result = CompletedRequestsHead;
	CompletedRequestsHead = CompletedRequestsTail = nullptr;
	return Result;
}

const TCHAR* FStorageServerIoDispatcherBackend::GetName() const
{
	return TEXT("StorageServer");
}

void FStorageServerIoDispatcherBackend::SubmitBatch(FBatch* Batch)
{
 	GIOThreadPool->AddQueuedWork(Batch);
	++SubmittedBatchesCount;
}

bool FStorageServerIoDispatcherBackend::WaitForBatchToComplete(uint32 WaitTime)
{
	bool bAtLeastOneCompleted = false;
	while (!bAtLeastOneCompleted)
	{
		if (!BatchCompletedEvent->Wait(WaitTime))
		{
			return false;
		}

		FBatch* LocalCompletedBatches;
		{
			FScopeLock _(&CompletedBatchesCritical);
			LocalCompletedBatches = FirstCompletedBatch;
			FirstCompletedBatch = nullptr;
		}
		while (LocalCompletedBatches)
		{
			check(SubmittedBatchesCount > 0);
			--SubmittedBatchesCount;
			FBatch* CompletedBatch = LocalCompletedBatches;
			LocalCompletedBatches = LocalCompletedBatches->Next;
			CompletedBatch->Next = FirstAvailableBatch;
			FirstAvailableBatch = CompletedBatch;
			bAtLeastOneCompleted = true;
		}
	}
	return true;
}

void FStorageServerIoDispatcherBackend::OnBatchCompleted(FBatch* Batch)
{
	{
		FScopeLock Lock(&CompletedRequestsCritical);
		if (CompletedRequestsTail)
		{
			CompletedRequestsTail->NextRequest = Batch->RequestsHead;
		}
		else
		{
			CompletedRequestsHead = Batch->RequestsHead;
		}
		CompletedRequestsTail = Batch->RequestsTail;
	}
	BackendContext->WakeUpDispatcherThreadDelegate.Execute();

	Batch->RequestsHead = Batch->RequestsTail = nullptr;
	Batch->RequestsCount = 0;
	{
		FScopeLock _(&CompletedBatchesCritical);
		Batch->Next = FirstCompletedBatch;
		FirstCompletedBatch = Batch;
	}
	BatchCompletedEvent->Trigger();
}

FStorageServerIoDispatcherBackend::FBatch::FBatch(FStorageServerIoDispatcherBackend& InOwner)
	: Owner(InOwner)
{

}

void FStorageServerIoDispatcherBackend::FBatch::DoThreadedWork()
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/StorageServer"));
	TRACE_CPUPROFILER_EVENT_SCOPE(StorageServerIoDispatcherProcessBatch);
	FIoRequestImpl* Request = RequestsHead;
#if 0
	TArray<FIoRequestImpl*, TInlineAllocator<64>> RequestsArray;
	FStorageServerChunkBatchRequest ChunkBatchRequest = Owner.Connection.NewChunkBatchRequest();
	while (Request)
	{
		TRACE_IOSTORE_BACKEND_REQUEST_STARTED(Request, &Owner);
		ChunkBatchRequest.AddChunk(Request->ChunkId, Request->Options.GetOffset(), Request->Options.GetSize());
		RequestsArray.Add(Request);
		Request = Request->NextRequest;
	}

	bool bSuccess = ChunkBatchRequest.Issue([&RequestsArray](uint32 ChunkCount, uint32* ChunkIndices, uint64* ChunkSizes, FStorageServerResponse& ChunkDataStream)
		{
			check(ChunkCount == RequestsArray.Num());
			for (uint32 ChunkIndex = 0; ChunkIndex < ChunkCount; ++ChunkIndex)
			{
				uint32 RequestIndex = ChunkIndices[ChunkIndex];
				FIoRequestImpl* Request = RequestsArray[RequestIndex];
				uint64 RequestSize = ChunkSizes[ChunkIndex];
				if (RequestSize == uint64(-1))
				{
					Request->SetFailed();
					TRACE_IOSTORE_BACKEND_REQUEST_FAILED(Request);
					continue;
				}
				if (void* TargetVa = Request->Options.GetTargetVa())
				{
					Request->IoBuffer = FIoBuffer(FIoBuffer::Wrap, TargetVa, RequestSize);
				}
				else
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(AllocMemoryForRequest);
					Request->IoBuffer = FIoBuffer(RequestSize);
				}
				TRACE_CPUPROFILER_EVENT_SCOPE(SerializeResponse);
				ChunkDataStream.Serialize(Request->IoBuffer.Data(), RequestSize);
				TRACE_IOSTORE_BACKEND_REQUEST_COMPLETED(Request, RequestSize);
			}
		});
	if (!bSuccess)
	{
		for (FIoRequestImpl* FailedRequest : RequestsArray)
		{
			FailedRequest->SetFailed();
			TRACE_IOSTORE_BACKEND_REQUEST_FAILED(FailedRequest);
		}
	}
#else
	while (Request)
	{
		FIoRequestImpl* NextRequest = Request->NextRequest;

		TRACE_IOSTORE_BACKEND_REQUEST_STARTED(Request, &Owner);
		TRACE_CPUPROFILER_EVENT_SCOPE(StorageServerIoDispatcherReadChunk);
		const TOptional<FIoBuffer> OptDestination = Request->Options.GetTargetVa() ? FIoBuffer(FIoBuffer::Wrap, Request->Options.GetTargetVa(), Request->Options.GetSize()) : TOptional<FIoBuffer>();
		const bool bHardwareTargetBuffer = EnumHasAnyFlags(Request->Options.GetFlags(), EIoReadOptionsFlags::HardwareTargetBuffer);
		TIoStatusOr<FIoBuffer> Result = Owner.Connection.ReadChunkRequest(Request->ChunkId, Request->Options.GetOffset(), Request->Options.GetSize(), OptDestination, bHardwareTargetBuffer); 
		if (Result.IsOk())
		{
			Request->SetResult(Result.ValueOrDie());
			TRACE_IOSTORE_BACKEND_REQUEST_COMPLETED(Request, Result.ValueOrDie().GetSize());
		}
		else
		{
			Request->SetFailed();
			TRACE_IOSTORE_BACKEND_REQUEST_FAILED(Request);
		}
		Request = NextRequest;
	}
#endif
	Owner.OnBatchCompleted(this);
}

FIoRequestImpl* FStorageServerIoDispatcherBackend::FRequestQueue::Pop()
{
	FScopeLock _(&CriticalSection);
	if (Heap.Num() == 0)
	{
		return nullptr;
	}
	FIoRequestImpl* Result;
	Heap.HeapPop(Result, QueueSortFunc, EAllowShrinking::No);
	return Result;
}

void FStorageServerIoDispatcherBackend::FRequestQueue::Push(FIoRequestImpl& Request)
{
	FScopeLock _(&CriticalSection);
	Heap.HeapPush(&Request, QueueSortFunc);
}

void FStorageServerIoDispatcherBackend::FRequestQueue::UpdateOrder()
{
	FScopeLock _(&CriticalSection);
	Heap.Heapify(QueueSortFunc);
}

#endif
