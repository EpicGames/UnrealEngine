// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/IoDispatcherBackend.h"
#include "HAL/Runnable.h"
#include "Misc/IQueuedWork.h"

#if !UE_BUILD_SHIPPING

class FStorageServerConnection;
struct FStorageServerSerializationContext;

class FStorageServerIoDispatcherBackend final
	: public FRunnable
	, public IIoDispatcherBackend
{
public:
	FStorageServerIoDispatcherBackend(FStorageServerConnection& Connection);
	~FStorageServerIoDispatcherBackend();

	void Initialize(TSharedRef<const FIoDispatcherBackendContext> Context) override;
	void Shutdown() override;
	void ResolveIoRequests(FIoRequestList Requests, FIoRequestList& OutUnresolved) override;
	void CancelIoRequest(FIoRequestImpl* Request) override {};
	void UpdatePriorityForIoRequest(FIoRequestImpl* Request) override {};
	bool DoesChunkExist(const FIoChunkId& ChunkId) const override;
	TIoStatusOr<uint64> GetSizeForChunk(const FIoChunkId& ChunkId) const override;
	FIoRequestImpl* GetCompletedIoRequests() override;
	TIoStatusOr<FIoMappedRegion> OpenMapped(const FIoChunkId& ChunkId, const FIoReadOptions& Options) override
	{
		return FIoStatus(EIoErrorCode::NotFound);
	}
	const TCHAR* GetName() const override;

	virtual bool Init() override
	{
		return true;
	}
	virtual uint32 Run() override;
	virtual void Stop() override;

private:
	class FRequestQueue
	{
	public:
		FIoRequestImpl* Pop();
		void Push(FIoRequestImpl& Request);
		void UpdateOrder();

	private:
		static bool QueueSortFunc(const FIoRequestImpl& A, const FIoRequestImpl& B)
		{
			uintptr_t ASequence = reinterpret_cast<uintptr_t>(A.BackendData);
			uintptr_t BSequence = reinterpret_cast<uintptr_t>(B.BackendData);
			if (A.Priority == B.Priority)
			{
				return ASequence < BSequence;
			}
			return A.Priority > B.Priority;
		}

		TArray<FIoRequestImpl*> Heap;
		FCriticalSection CriticalSection;
	};

	struct FBatch
		: public IQueuedWork
	{
		FBatch(FStorageServerIoDispatcherBackend& InOwner);

		virtual void DoThreadedWork() override;
		virtual void Abandon() override {};

		FStorageServerIoDispatcherBackend& Owner;
		FBatch* Next = nullptr;
		FIoRequestImpl* RequestsHead = nullptr;
		FIoRequestImpl* RequestsTail = nullptr;
		uint64 RequestsCount = 0;
	};

	bool Resolve(FIoRequestImpl* Request);
	void SubmitBatch(FBatch* Batch);
	void OnBatchCompleted(FBatch* Batch);
	bool WaitForBatchToComplete(uint32 WaitTime = MAX_uint32);

	FStorageServerConnection& Connection;
	TSharedPtr<const FIoDispatcherBackendContext> BackendContext;
	FRunnableThread* Thread = nullptr;
	FEvent* NewRequestEvent;
	FRequestQueue RequestQueue;
	FBatch* FirstAvailableBatch = nullptr;
	int32 SubmittedBatchesCount = 0;
	FCriticalSection CompletedBatchesCritical;
	FBatch* FirstCompletedBatch = nullptr;
	FEvent* BatchCompletedEvent;
	FCriticalSection CompletedRequestsCritical;
	FIoRequestImpl* CompletedRequestsHead = nullptr;
	FIoRequestImpl* CompletedRequestsTail = nullptr;
	TAtomic<bool> bStopRequested{ false };
};

#endif
