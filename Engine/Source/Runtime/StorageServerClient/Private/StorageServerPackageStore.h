// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/PackageStore.h"
#include "Templates/SharedPointer.h"
#include "HAL/Runnable.h"

#if !UE_BUILD_SHIPPING

class FStorageServerConnection;
struct FFilePackageStoreEntry;

class FStorageServerPackageStoreBackend
	: public IPackageStoreBackend
{
public:
	FStorageServerPackageStoreBackend(FStorageServerConnection& Connection);
	virtual ~FStorageServerPackageStoreBackend() = default;

	virtual void OnMounted(TSharedRef<const FPackageStoreBackendContext> Context) override
	{
	}

	virtual void BeginRead() override
	{
	}

	virtual void EndRead() override
	{
	}

	virtual EPackageStoreEntryStatus GetPackageStoreEntry(FPackageId PackageIde, FName PackageName,
		FPackageStoreEntry& OutPackageStoreEntry) override;
	
	virtual bool GetPackageRedirectInfo(FPackageId PackageId, FName& OutSourcePackageName, FPackageId& OutRedirectedToPackageId) override
	{
		return false;
	}

private:
	struct FStoreEntry
	{
		TArray<FPackageId> ImportedPackages;
		TArray<FShaderHash> ShaderMapHashes;
#if WITH_EDITOR
		EPackageStoreEntryFlags Flags;
		TArray<FPackageId> OptionalSegmentImportedPackageIds;
#endif
	};
	TMap<FPackageId, FStoreEntry> StoreEntriesMap;

	class FAsyncInitRunnable : public FRunnable
	{
	public:
		FAsyncInitRunnable(FStorageServerPackageStoreBackend& InOwner, FStorageServerConnection& InConnection);
		virtual ~FAsyncInitRunnable();
		void WaitForCompletion() const;

	private:
		virtual uint32 Run() override;

		FStorageServerPackageStoreBackend& Owner;
		FStorageServerConnection& Connection;
		class FEvent* IsCompleted;
	};
	TSharedPtr<FAsyncInitRunnable, ESPMode::ThreadSafe> AsyncInit;
};

#endif