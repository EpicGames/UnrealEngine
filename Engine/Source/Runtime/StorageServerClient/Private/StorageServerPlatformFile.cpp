// Copyright Epic Games, Inc. All Rights Reserved.

#include "StorageServerPlatformFile.h"
#include "Algo/Replace.h"
#include "Async/ManualResetEvent.h"
#include "CookOnTheFly.h"
#include "CookOnTheFlyPackageStore.h"
#include "HAL/FileManagerGeneric.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/CriticalSection.h"
#include "Misc/App.h"
#include "Misc/Base64.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/StringBuilder.h"
#include "Modules/ModuleManager.h"
#include "Modules/ModuleManager.h"
#include "Serialization/CompactBinaryPackage.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "ProfilingDebugging/PlatformFileTrace.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "StorageServerConnection.h"
#include "StorageServerIoDispatcherBackend.h"
#include "StorageServerPackageStore.h"
#include "Containers/LruCache.h"
#include "Containers/SpscQueue.h"
#include <atomic>
#include "Async/MappedFileHandle.h"
#include "CoreGlobalsInternal.h"

DEFINE_LOG_CATEGORY_STATIC(LogStorageServerPlatformFile, Log, All);

#if !UE_BUILD_SHIPPING

#ifndef EXCLUDE_NONSERVER_UE_EXTENSIONS
#define EXCLUDE_NONSERVER_UE_EXTENSIONS 1	// Use .Build.cs file to disable this if the game relies on accessing loose files on the local filesystem
#endif


#if !defined(WITH_STORAGE_SERVER_STARTUP_FILE_CACHE)
#define WITH_STORAGE_SERVER_STARTUP_FILE_CACHE 1
#endif


#if !defined(HAS_STORAGE_SERVER_COMPRESSED_FILE_HANDLE)
#	define HAS_STORAGE_SERVER_COMPRESSED_FILE_HANDLE 0
#endif

#if HAS_STORAGE_SERVER_COMPRESSED_FILE_HANDLE
	IWrappedFileHandle* CreateCompressedPlatformFileHandle(IFileHandle* InLowerLevelHandle);
#else
	IWrappedFileHandle* CreateCompressedPlatformFileHandle(IFileHandle* InLowerLevelHandle)
	{
		return nullptr;
	}
#endif // HAS_STORAGE_SERVER_COMPRESSED_FILE_HANDLE

static FDateTime GAssumedImmutableTimeStamp = FDateTime::Now();

// If this is set, then StorageServer will not be used for non-assets (OpenRead, etc is used for non-assets; assets are handled via the IoDispatcher)
static bool GPreferLocalForNonAssets = false;
static FAutoConsoleVariableRef CVarPreferLocalForNonAssets(
	TEXT("s.PreferLocalForNonAssets"),
	GPreferLocalForNonAssets,
	TEXT("Set to true to look at the local file sytem for files before loading from StorageServer"),
	ECVF_Default
);

static bool IsFileInDynamicPath(const FString InFilename)
{
	return FPaths::IsUnderDirectory(InFilename, FPaths::ProjectSavedDir()) ||
		FPaths::IsUnderDirectory(InFilename, FPaths::EngineSavedDir()) ||
		FPaths::IsUnderDirectory(InFilename, FPaths::GeneratedConfigDir()) ||
		FPaths::IsUnderDirectory(InFilename, FPaths::GameAgnosticSavedDir()) ||
		FPaths::IsUnderDirectory(InFilename, FPaths::EngineVersionAgnosticUserDir()) ||
		FPaths::IsUnderDirectory(InFilename, FPaths::EngineUserDir()) ||
		FPaths::IsUnderDirectory(InFilename, FPaths::ProjectUserDir()) ||
		FPaths::IsUnderDirectory(InFilename, FPaths::EngineIntermediateDir()) ||
		FPaths::IsUnderDirectory(InFilename, FPaths::ProjectIntermediateDir()) ||
		FPaths::IsUnderDirectory(InFilename, FPaths::CloudDir()) ||
		FPaths::IsUnderDirectory(InFilename, FPaths::GameDevelopersDir()) ||
		(FPaths::HasProjectPersistentDownloadDir() && FPaths::IsUnderDirectory(InFilename, FPaths::ProjectPersistentDownloadDir()));
}


FStorageServerFileSystemTOC::~FStorageServerFileSystemTOC()
{
	FWriteScopeLock _(TocLock);
	for (auto& KV : Directories)
	{
		delete KV.Value;
	}
}

FStorageServerFileSystemTOC::FDirectory* FStorageServerFileSystemTOC::AddDirectoriesRecursive(const FString& DirectoryPath)
{
	FDirectory* Directory = new FDirectory();
	Directories.Add(DirectoryPath, Directory);
	FString ParentDirectoryPath = FPaths::GetPath(DirectoryPath);
	FDirectory* ParentDirectory;
	if (ParentDirectoryPath.IsEmpty())
	{
		ParentDirectory = &Root;
	}
	else
	{
		ParentDirectory = Directories.FindRef(ParentDirectoryPath);
		if (!ParentDirectory)
		{
			ParentDirectory = AddDirectoriesRecursive(ParentDirectoryPath);
		}
	}
	ParentDirectory->Directories.Add(DirectoryPath);
	return Directory;
}

void FStorageServerFileSystemTOC::Clear()
{
	FWriteScopeLock _(TocLock);
	for (auto& KV : Directories)
	{
		delete KV.Value;
	}

	Root.Directories.Empty();
	Root.Files.Empty();
	Files.Empty();
	Directories.Empty();
	FilePathToIndexMap.Empty();
}

void FStorageServerFileSystemTOC::AddFile(const FIoChunkId& FileChunkId, FStringView PathView, int64 RawSize)
{
	FWriteScopeLock _(TocLock);

	const int32 FileIndex = Files.Num();
	
	FFile& NewFile = Files.AddDefaulted_GetRef();
	NewFile.FileChunkId = FileChunkId;
	NewFile.FilePath = PathView;
	NewFile.RawSize = RawSize;
	
	FilePathToIndexMap.Add(NewFile.FilePath, FileIndex);
	
	FString DirectoryPath = FPaths::GetPath(NewFile.FilePath);
	FDirectory* Directory = Directories.FindRef(DirectoryPath);
	if (!Directory)
	{
		Directory = AddDirectoriesRecursive(DirectoryPath);
	}
	Directory->Files.Add(FileIndex);
}

bool FStorageServerFileSystemTOC::FileExists(const FString& Path)
{
	FReadScopeLock _(TocLock);
	return FilePathToIndexMap.Contains(Path);
}

bool FStorageServerFileSystemTOC::DirectoryExists(const FString& Path)
{
	FReadScopeLock _(TocLock);
	return Directories.Contains(Path);
}

const FIoChunkId* FStorageServerFileSystemTOC::GetFileChunkId(const FString& Path)
{
	FReadScopeLock _(TocLock);
	if (const int32* FileIndex = FilePathToIndexMap.Find(Path))
	{
		return &Files[*FileIndex].FileChunkId;
	}
	return nullptr;
}

int64 FStorageServerFileSystemTOC::GetFileSize(const FString& Path)
{
	FReadScopeLock _(TocLock);
	if (const int32* FileIndex = FilePathToIndexMap.Find(Path))
	{
		return Files[*FileIndex].RawSize;
	}
	return STORAGE_SERVER_FILE_UNKOWN_SIZE;
}

bool FStorageServerFileSystemTOC::GetFileData(const FString& Path, FIoChunkId& OutChunkId, int64& OutRawSize)
{
	FReadScopeLock _(TocLock);
	if (const int32* FileIndex = FilePathToIndexMap.Find(Path))
	{
		const FFile& File = Files[*FileIndex];
		OutChunkId = File.FileChunkId;
		OutRawSize = File.RawSize;
		return true;
	}
	return false;
}

bool FStorageServerFileSystemTOC::IterateDirectory(const FString& Path, TFunctionRef<bool(const FIoChunkId&, const TCHAR*, int64 RawSize)> Callback)
{
	UE_LOGF(LogStorageServerPlatformFile, Verbose, "IterateDirectory '%ls'", *Path);

	FReadScopeLock _(TocLock);

	FDirectory* Directory = Directories.FindRef(Path);
	if (!Directory)
	{
		return false;
	}
	for (int32 FileIndex : Directory->Files)
	{
		const FFile& File = Files[FileIndex];
		if (!Callback(File.FileChunkId, *File.FilePath, File.RawSize))
		{
			return false;
		}
	}
	for (const FString& ChildDirectoryPath : Directory->Directories)
	{
		if (!Callback(FIoChunkId(), *ChildDirectoryPath, 0))
		{
			return false;
		}
	}
	return true;
}

bool FStorageServerFileSystemTOC::IterateDirectoryRecursively(const FString& Path, TFunctionRef<bool(const FIoChunkId&, const TCHAR*, int64)> Callback)
{
	UE_LOGF(LogStorageServerPlatformFile, Verbose, "IterateDirectoryRecursively '%ls'", *Path);

	FReadScopeLock _(TocLock);
	FDirectory* Directory = Directories.FindRef(Path);
	if (!Directory)
	{
		return false;
	}
	for (int32 FileIndex : Directory->Files)
	{
		const FFile& File = Files[FileIndex];
		if (!Callback(File.FileChunkId, *File.FilePath, File.RawSize))
		{
			return false;
		}
	}
	bool bFail = false;
	for (const FString& ChildDirectoryPath : Directory->Directories)
	{
		bFail |= !IterateDirectoryRecursively(ChildDirectoryPath, Callback);
	}

	return !bFail;
}





#if WITH_STORAGE_SERVER_STARTUP_FILE_CACHE

#define STORAGE_SERVER_START_CACHE_REPORT 0 // log out startup cache usage at the end of engine startup

class FStorageServerEngineStartupPrecache : public FRunnable
{
public:

	// largest file size that will be stored in the cache
	static const uint32 MaxFileSize = 16 * 1024;

	// largest size that the cache is allowed to grow to
	static const uint64 MaxCacheSize = 10 * 1024 * 1024;


	FStorageServerEngineStartupPrecache(FStorageServerConnection& InConnection)
		: Connection(InConnection)
		, CacheSize(0)
		, bExitWhenComplete(false)
		, bExitImmediately(false)
		, bUseBatchedRequests(false) // Turned off batch requests until a stall can be diagnosed and fixed
	{
		if (FRunnableThread::Create( this, TEXT("StorageServerPrecache"), 0, EThreadPriority::TPri_Normal) == nullptr)
		{
			IsCompleted.Notify();
		}

		if (Connection.IsConnectedToWorkspace())
		{
			bUseBatchedRequests = false;
		}
	}

	virtual ~FStorageServerEngineStartupPrecache()
	{
		// the worker thread has most likely completed already but no harm in shutting down cleanly
		bExitImmediately = true;
		HasWork.Notify();
		IsCompleted.Wait();
		IsCompleted.Reset();

#if STORAGE_SERVER_START_CACHE_REPORT
		UE_CLOGF(Cache.Num() > 0, LogStorageServerPlatformFile, Log, "Engine startup precache size was %llu bytes, %d items. %d items actually retrieved (%d%%). %d cache misses (%d%%)", CacheSize.load(), Cache.Num(), AccessedChunks.Num(), (AccessedChunks.Num()*100) / Cache.Num(), UncachedChunks.Num(), (UncachedChunks.Num()*100) / (AccessedChunks.Num()+UncachedChunks.Num()) );
#endif
	}

	void AddPrecachedFile( const FIoChunkId& Id, uint32 Size, bool bHighPriority )
	{
		if (bHighPriority)
		{
			HighPriorityPrecacheFileRequests.Enqueue( TTuple<FIoChunkId,int32>(Id, Size) );
		}
		else
		{
			PrecacheFileRequests.Enqueue( TTuple<FIoChunkId,int32>(Id, Size) );
		}
		HasWork.Notify();
	}

	void Finalize()
	{
		// no more precache requests after this - thread will shut down when precaching has finished
		bExitWhenComplete = true;
		HasWork.Notify();
	}

	bool GetPrecachedFile( const FIoChunkId& Id, int64 Offset, int64 Size, uint8* Destination, int64& BytesRead)
	{
		bool bSuccess = false;
		{
			FReadScopeLock Lock(CacheLock);
			if (const TArray<uint8>* BufferPtr = Cache.Find(Id))
			{
				if (ensure(Offset < BufferPtr->Num()))
				{
					BytesRead = FMath::Min(BufferPtr->Num() - Offset, Size);
					FMemory::Memcpy(Destination, BufferPtr->GetData() + Offset, BytesRead);
					bSuccess = true;
				}
			}
		}

#if STORAGE_SERVER_START_CACHE_REPORT
		if (bSuccess)
		{
			FWriteScopeLock Lock(CacheLock);
			AccessedChunks.Add(Id);
		}
		else
		{
			FWriteScopeLock Lock(CacheLock);
			UncachedChunks.Add(Id);
		}
#endif

		return bSuccess;
	}

private:
	virtual uint32 Run() override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(StorageServerPlatformEngineStartupPrecache);

		while (!bExitImmediately)
		{
			HasWork.Wait();
			HasWork.Reset();
			if (bExitImmediately)
			{
				break;
			}

#if HAS_STORAGE_SERVER_RPC_GETCHUNKS_API
			PrecacheItems();
			if (bExitWhenComplete && !bExitImmediately)
			{
				// Clear out the final batch of requests which may have been added after the previous
				// PrecacheItems() call had processed the queue, but before the HTTP response had
				// returned from the server.
				PrecacheItems();
				break;
			}
#endif
		}

		IsCompleted.Notify();
		return 0;
	}

	void HandleChunkBatchResponse(FIoChunkId ChunkId, EStorageServerContentType MimeType, FIoBuffer Data, const TOptional<uint64>& ModTag)
	{
		TArray<uint8> DecompressedData;

		// TODO move decompression to StorageServerConnection.
		if (MimeType == EStorageServerContentType::CompressedBinary)
		{
			FCompressedBuffer Buffer = FCompressedBuffer::FromCompressed(FSharedBuffer::MakeView(Data.GetView()));
			FCompressedBufferReader Reader(Buffer);

			DecompressedData.AddUninitialized(Reader.GetRawSize());

			if (!ensureMsgf(Reader.TryDecompressTo(FMutableMemoryView(DecompressedData.GetData(), DecompressedData.Num())), TEXT("Failed to decompress data from server response")))
			{
				return;
			}
		}
		else
		{
			DecompressedData.Append(Data.GetData(), Data.GetSize());
		}

		if (CacheSize + DecompressedData.Num() <= MaxCacheSize)
		{
			FWriteScopeLock Lock(CacheLock);
			CacheSize += DecompressedData.Num();
			Cache.Add(ChunkId, DecompressedData);
		}
	}

#if HAS_STORAGE_SERVER_RPC_GETCHUNKS_API
	void PrecacheItems()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FStorageServerEngineStartupPrecache::PrecacheItems);

		TTuple<FIoChunkId,int32> Item;
		uint32 CorrelationId = 0;

		TArray<FStorageServerConnection::FChunkBatchRequestEntry> Items;

		while (HighPriorityPrecacheFileRequests.Dequeue(Item) || PrecacheFileRequests.Dequeue(Item))
		{
			const FIoChunkId& Id = Item.Key;
			const int32& Size = Item.Value;

			if (bExitImmediately)
			{
				return;
			}

			// make sure the total cache size doesn't grow out of control
			if (CacheSize + Size > MaxCacheSize)
			{
				break;
			}

			if (bUseBatchedRequests)
			{
				Items.Add(FStorageServerConnection::FChunkBatchRequestEntry{Id, 0, uint64(Size)});
			}
			else
			{
				TArray<uint8> TempBuffer;
				TempBuffer.AddUninitialized(Size);
				TIoStatusOr<FIoBuffer> Result = Connection.ReadChunkRequest(Id, 0, Size, FIoBuffer(FIoBuffer::Wrap, TempBuffer.GetData(), Size), false);
				if (Result.IsOk())
				{
					TempBuffer.SetNum(Result.ValueOrDie().GetSize());
					CacheSize += TempBuffer.Num();
					{
						FWriteScopeLock Lock(CacheLock);
						Cache.Add(Id, MoveTemp(TempBuffer));
					}
				}
			}
		}

		if (Items.IsEmpty())
		{
			return;
		}

		FIoStatus ResponseStatus = Connection.ReadChunkBatchRequest(Items, [&](FIoChunkId Id, EStorageServerContentType MimeType, FIoBuffer Data, const TOptional<uint64>& ModTag)
		{
			HandleChunkBatchResponse(Id, MimeType, Data, ModTag);
		});

		if (!ensureMsgf(ResponseStatus.IsOk(), TEXT("Failed to read chunk batch request from Zen server")))
		{
			return;
		}
	}
#endif

	FStorageServerConnection& Connection;

	FRWLock CacheLock;
	TMap<FIoChunkId,TArray<uint8>> Cache;
	std::atomic<uint64> CacheSize;

	TSpscQueue<TTuple<FIoChunkId,int32>> PrecacheFileRequests;
	TSpscQueue<TTuple<FIoChunkId,int32>> HighPriorityPrecacheFileRequests;
	UE::FManualResetEvent HasWork;
	UE::FManualResetEvent IsCompleted;

	std::atomic<bool> bExitWhenComplete;
	std::atomic<bool> bExitImmediately;
	std::atomic<bool> bUseBatchedRequests;
	
#if STORAGE_SERVER_START_CACHE_REPORT
	TSet<FIoChunkId> UncachedChunks;
	TSet<FIoChunkId> AccessedChunks;
#endif
};


TUniquePtr<FStorageServerEngineStartupPrecache> GStorageServerEngineStartupPrecache;

#endif // WITH_STORAGE_SERVER_STARTUP_FILE_CACHE







#if COUNTERSTRACE_ENABLED
	TRACE_DECLARE_ATOMIC_FLOAT_COUNTER(StorageServerCache_HitRatioBytes, TEXT("ZenClient/FileCache/HitRatio"));
	namespace
	{
		static std::atomic<uint64> CacheHitBytes = 0;
		static std::atomic<uint64> CacheMissBytes = 0;
	}

	#define STORAGESERVER_CACHEMISS(Bytes) \
	{\
		CacheMissBytes += Bytes; \
		TRACE_COUNTER_SET(StorageServerCache_HitRatioBytes, (double)CacheHitBytes / (double)(CacheMissBytes+CacheHitBytes) ); \
	}

	#define STORAGESERVER_CACHEHIT(Bytes) \
	{\
		CacheHitBytes += Bytes; \
		TRACE_COUNTER_SET(StorageServerCache_HitRatioBytes, (double)CacheHitBytes / (double)(CacheMissBytes+CacheHitBytes) ); \
	}

#else

	#define STORAGESERVER_CACHEMISS(Bytes)
	#define STORAGESERVER_CACHEHIT(Bytes)

#endif // COUNTERSTRACE_ENABLED



class FStorageServerFileCache
{
private:
	typedef FIoChunkId CacheKey;

	typedef DefaultKeyComparer<FIoChunkId> CacheKeyComparer;
public:
	// zen compression block size is often 256kb
	static const int64 BlockSize = 256 * 1024;

	// up to 4 mb cache, not counting temporary read buffers
	static const uint32 MaxCacheElements = 16; 


	struct CacheEntry
	{
		int64 Start = -1;
		TArray<uint8, TInlineAllocator<BlockSize>> Buffer;

		FORCEINLINE int64 End()
		{
			return Start + Buffer.Num();
		}

		bool TryReadFromCache(int64& FilePos, uint8*& Destination, int64& BytesToRead, int64& BytesRead)
		{
			if (FilePos >= Start && FilePos < End())
			{
				BytesRead = FMath::Min(End() - FilePos, BytesToRead);
				FMemory::Memcpy(Destination, Buffer.GetData() + FilePos - Start, BytesRead);
				FilePos += BytesRead;
				Destination += BytesRead;
				BytesToRead -= BytesRead;
				return true;
			}
			else
			{
				return false;
			}
		}
	};

	static FORCEINLINE int64 BlockOffset(int64 Position)
	{
		return (Position / BlockSize) * BlockSize;
	}

	static FStorageServerFileCache& Get()
	{
		static FStorageServerFileCache Instance;
		return Instance;
	}

	void Lock()
	{
		CriticalSection.Lock();
	}

	void Unlock()
	{
		CriticalSection.Unlock();
	}

	CacheEntry& FindOrAdd(FIoChunkId FileChunkId)
	{
		CacheKey Key = FileChunkId;
		if (const CacheEntry* ExistingEntry = Cache.FindAndTouch(Key))
		{
			return *const_cast<CacheEntry*>(ExistingEntry); // TODO change LRU cache API
		}
		else
		{
			CacheEntry& Entry = Cache.AddUninitialized_GetRef(Key);
			Entry.Start = -1;
			Entry.Buffer.Empty();

			return Entry;
		}
	}

	void ReadCached(FStorageServerConnection* Connection, FIoChunkId FileChunkId, int64& FilePos, uint8*& Destination, int64& BytesToRead)
	{
		if (BytesToRead == 0)
		{
			return;
		}

#if WITH_STORAGE_SERVER_STARTUP_FILE_CACHE
		// check engine startup cache
		if ( FStorageServerEngineStartupPrecache* Precache = GStorageServerEngineStartupPrecache.Get() )
		{
			int64 BytesRead;
			if (Precache->GetPrecachedFile( FileChunkId, FilePos, BytesToRead, Destination, BytesRead ))
			{
				STORAGESERVER_CACHEHIT(BytesRead);
				BytesToRead -= BytesRead;
				FilePos += BytesRead;
				Destination += BytesRead;
				return;
			}
		}
#endif


		// try to read existing data from cache
		{
			UE::TScopeLock Lock(*this);

			CacheEntry& Entry = FindOrAdd(FileChunkId);
			int64 BytesRead = 0;
			if (Entry.TryReadFromCache(FilePos, Destination, BytesToRead, BytesRead))
			{
				STORAGESERVER_CACHEHIT(BytesRead);
			}

			if (BytesToRead == 0)
			{
				return;
			}
		}


		// if request spans multiple blocks, satisfy all but last block without cache 
		if (BlockOffset(FilePos) < BlockOffset(FilePos + BytesToRead))
		{
			const int64 BytesToReadRequested = BlockOffset(BytesToRead + FilePos) - FilePos;
			const int64 BytesRead = SendReadMessage(Connection, Destination, FileChunkId, FilePos, BytesToReadRequested);
			STORAGESERVER_CACHEMISS(BytesRead);
			FilePos += BytesRead;
			Destination += BytesRead;
			BytesToRead -= BytesRead;
		}

		if (BytesToRead == 0)
		{
			return;
		}

		// try to read last block from cache
		{
			UE::TScopeLock Lock(*this);

			CacheEntry& Entry = FindOrAdd(FileChunkId);
			int64 BytesRead = 0;
			if (Entry.TryReadFromCache(FilePos, Destination, BytesToRead, BytesRead))
			{
				STORAGESERVER_CACHEHIT(BytesRead);
				if (ensure(BytesToRead == 0))
				{
					return;
				}
			}

		}

		// read and cache last block
		// TODO try to avoid doing two requests for large reads 
		{
			TArray<uint8> TempBuffer; // allocating a temporary BlockSize buffer here for the read - one per parallel file access
			TempBuffer.AddUninitialized(BlockSize);
			int64 TempStart = BlockOffset(FilePos);

			int64 BytesRead = SendReadMessage(Connection, TempBuffer.GetData(), FileChunkId, TempStart, TempBuffer.Num());
			STORAGESERVER_CACHEMISS(BytesRead);

			{
				UE::TScopeLock Lock(*this);

				CacheEntry& Entry = FindOrAdd(FileChunkId);
				Entry.Start = TempStart;
				Entry.Buffer.SetNum(BytesRead);
				FMemory::Memcpy(Entry.Buffer.GetData(), TempBuffer.GetData(), BytesRead);

				ensure(Entry.TryReadFromCache(FilePos, Destination, BytesToRead, BytesRead));
			}
		}

		check(BytesToRead == 0);
	}

private:
	FStorageServerFileCache()
		: Cache(MaxCacheElements)
	{
	}

	int64 SendReadMessage(FStorageServerConnection* Connection, uint8* Destination, const FIoChunkId& FileChunkId, int64 Offset, int64 BytesToRead)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FStorageServerFileCache::SendReadMessage);
		int64 BytesRead = 0;
		TIoStatusOr<FIoBuffer> Result = Connection->ReadChunkRequest(FileChunkId, Offset, BytesToRead, FIoBuffer(FIoBuffer::Wrap, Destination, BytesToRead), false);
		BytesRead = Result.IsOk() ? Result.ValueOrDie().GetSize() : 0;
		return BytesRead;
	}

	TLruCache<CacheKey, CacheEntry, CacheKeyComparer> Cache;
	FCriticalSection CriticalSection;
};

class FStorageServerFileHandle
	: public IFileHandle
{
	enum
	{
		BufferSize = 64 << 10
	};
	FStorageServerPlatformFile& Owner;
	FIoChunkId FileChunkId;
	FString Filename;
	int64 FilePos = 0;
	int64 FileSize = -1;
	int64 BufferStart = -1;
	int64 BufferEnd = -1;
	uint8 Buffer[BufferSize];
	FCriticalSection BufferCS;

public:
	FStorageServerFileHandle(FStorageServerPlatformFile& InOwner, FIoChunkId InFileChunkId, int64 InFileSize, const TCHAR* InFilename)
		: Owner(InOwner)
		, FileChunkId(InFileChunkId)
		, Filename(InFilename)
		, FileSize(InFileSize)
	{
		TRACE_PLATFORMFILE_BEGIN_OPEN(*FString::Printf(TEXT("zen:%s"), InFilename));
		TRACE_PLATFORMFILE_END_OPEN(this);
	}

	~FStorageServerFileHandle()
	{
		TRACE_PLATFORMFILE_BEGIN_CLOSE(this);
		TRACE_PLATFORMFILE_END_CLOSE(this);
	}

	virtual int64 Size() override
	{
		if (FileSize < 0)
		{
			const FFileStatData FileStatData = Owner.SendGetStatDataMessage(FileChunkId);
			if (FileStatData.bIsValid)
			{
				FileSize = FileStatData.FileSize;
			}
			else
			{
				UE_LOGF(LogStorageServerPlatformFile, Warning, "Failed to obtain size of file '%ls'", *Filename);
				FileSize = 0;
			}
		}
		return FileSize;
	}

	virtual int64 Tell() override
	{
		return FilePos;
	}

	virtual bool Seek(int64 NewPosition) override
	{
		FilePos = NewPosition;
		return true;
	}

	virtual bool SeekFromEnd(int64 NewPositionRelativeToEnd = 0) override
	{
		return Seek(Size() + NewPositionRelativeToEnd);
	}

	virtual bool Read(uint8* Destination, int64 BytesToRead) override
	{
		TRACE_PLATFORMFILE_BEGIN_READ(Destination, this, FilePos, BytesToRead);
		if (BytesToRead == 0)
		{
			TRACE_PLATFORMFILE_END_READ(Destination, 0);
			return true;
		}

		FStorageServerFileCache& Cache = FStorageServerFileCache::Get();

		uint8* DestinationPtr = Destination;
		int64 BytesRemaining = BytesToRead;
		Cache.ReadCached(Owner.Connection.Get(), FileChunkId, /*out*/FilePos, /*out*/DestinationPtr, /*out*/BytesRemaining);
		int64 BytesRead = (BytesToRead - BytesRemaining);

		TRACE_PLATFORMFILE_END_READ(Destination, BytesRead);
		
		return BytesRemaining == 0;
	}

	virtual bool ReadAt(uint8* Destination, int64 BytesToRead, int64 Offset)
	{
		if (BytesToRead == 0)
		{
			return true;
		}

		if (BytesToRead > BufferSize)
		{
			const int64 BytesRead = Owner.SendReadMessage(Destination, FileChunkId, Offset, BytesToRead);
			if (BytesRead == BytesToRead)
			{
				STORAGESERVER_CACHEMISS(BytesRead);
				return true;
			}
			return false;
		}

		{
			FScopeLock BufferLock(&BufferCS);

			int64 BytesReadFromBuffer = 0;
			if (Offset >= BufferStart && Offset < BufferEnd)
			{
				const int64 BufferOffset = Offset - BufferStart;
				check(BufferOffset < BufferSize);
				BytesReadFromBuffer = FMath::Min(BufferSize - BufferOffset, BytesToRead);
				FMemory::Memcpy(Destination, Buffer + BufferOffset, BytesReadFromBuffer);
				STORAGESERVER_CACHEHIT(BytesReadFromBuffer);
				if (BytesReadFromBuffer == BytesToRead)
				{
					Offset += BytesReadFromBuffer;
					return true;
				}
			}

			const int64 BytesRead = Owner.SendReadMessage(Buffer, FileChunkId, Offset + BytesReadFromBuffer, BufferSize);
			BufferStart = Offset + BytesReadFromBuffer;
			BufferEnd = BufferStart + BytesRead;

			const int64 BytesToReadFromBuffer = FMath::Min(BytesRead, BytesToRead - BytesReadFromBuffer);
			FMemory::Memcpy(Destination + BytesReadFromBuffer, Buffer, BytesToReadFromBuffer);
			BytesReadFromBuffer += BytesToReadFromBuffer;
			if (BytesReadFromBuffer == BytesToRead)
			{
				Offset += BytesReadFromBuffer;
				STORAGESERVER_CACHEMISS(BytesReadFromBuffer);
				return true;
			}

			return false;
		}
	}

	virtual bool Write(const uint8* Source, int64 BytesToWrite) override
	{
		check(false);
		return false;
	}

	virtual bool Flush(const bool bFullFlush = false) override
	{
		return false;
	}

	virtual bool Truncate(int64 NewSize) override
	{
		return false;
	}
};

FStorageServerPlatformFile::FStorageServerPlatformFile()
{
	if (UE::IsUsingZenPakFileStreaming())
	{
		ServerEngineDirView = FStringView(TEXT("Engine/"));
		ServerProjectDirView = FStringView(TEXT(UE_STRINGIZE(UE_PROJECT_NAME)) TEXT("/"));
	}
}

FStorageServerPlatformFile::~FStorageServerPlatformFile()
{
}

static FString GetCookedPlatformName()
{
	FString PlatformName;
	if (IsRunningHybridCookedEditor())
	{
#if UE_SERVER
		const TCHAR* Key = TEXT("RuntimeServerPlatform");
#else
		const TCHAR* Key = FParse::Param(FCommandLine::Get(), TEXT("server")) ? TEXT("RuntimeServerPlatform") : TEXT("RuntimeTargetPlatform");
#endif

		// manually look in DefaultEngine.ini, or BaseEngine.ini, for the name of the platform to use to find the ue.projectstore file
		FConfigFile Default;
		Default.Read(*(FPaths::ProjectConfigDir() / TEXT("DefaultEngine.ini")));
		if (!Default.GetString(TEXT("HybridCookedEditor"), Key, PlatformName))
		{
			FConfigFile Base;
			Base.Read(*(FPaths::EngineConfigDir() / TEXT("BaseEngine.ini")));
			// we expect this to always be found
			verify(Base.GetString(TEXT("HybridCookedEditor"), Key, PlatformName));
		}
		PlatformName.ReplaceInline(TEXT("{Platform}"), ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName()));
	}
	else
	{
		PlatformName = FPlatformProperties::PlatformName();
	}

	return PlatformName;
}

TUniquePtr<FArchive> FStorageServerPlatformFile::TryFindProjectStoreMarkerFile(IPlatformFile* Inner) const
{
	if (Inner == nullptr)
	{
		return nullptr;
	}

	TArray<FString> PotentialProjectStorePaths;
	if (CustomProjectStorePath.IsEmpty())
	{
		FString RelativeStagedPath = "../../../";
		FString RootPath = FPaths::RootDir();
		FString PlatformName = GetCookedPlatformName();
		FString CookedOutputPath = FPaths::Combine(FPaths::ProjectDir(), TEXT("Saved"), TEXT("Cooked"), PlatformName);

		PotentialProjectStorePaths.Add(RelativeStagedPath);
		PotentialProjectStorePaths.Add(CookedOutputPath);
		PotentialProjectStorePaths.Add(RootPath);
	}
	else
	{
		PotentialProjectStorePaths.Add(CustomProjectStorePath);
	}

	for (const FString& ProjectStorePath : PotentialProjectStorePaths)
	{
		FString ProjectMarkerPath = ProjectStorePath / TEXT("ue.projectstore");
		if (IFileHandle* ProjectStoreMarkerHandle = Inner->OpenRead(*ProjectMarkerPath); ProjectStoreMarkerHandle != nullptr)
		{
			UE_LOGF(LogStorageServerPlatformFile, Display, "Found '%ls'", *ProjectMarkerPath);
			return TUniquePtr<FArchive>(new FArchiveFileReaderGeneric(ProjectStoreMarkerHandle, *ProjectMarkerPath, ProjectStoreMarkerHandle->Size()));
		}
	}
	return nullptr;
}

static FAnsiString FindWorkspaceSharePath(const FString& WorkspaceSharePath, const FStorageServerConnection::Workspaces& Workspaces)
{
	const bool IsRelative = FPaths::IsRelative(WorkspaceSharePath);
	for (const auto& Workspace : Workspaces.Workspaces)
	{
		if (!Workspace.Root.IsEmpty())
		{
			FString WorkspaceRoot = Workspace.Root;
			FPaths::NormalizeDirectoryName(WorkspaceRoot);
			for (const auto& Share : Workspace.Shares)
			{
				FString SharePath(Share.Path);
				FPaths::NormalizeDirectoryName(SharePath);
				if (FPaths::IsSamePath(IsRelative ? SharePath : FPaths::Combine(WorkspaceRoot, SharePath), WorkspaceSharePath))
				{
					TAnsiStringBuilder<256> SharePathBuilder;
					SharePathBuilder.Append("/ws/");
					SharePathBuilder.Append(Workspace.Id);
					SharePathBuilder.Append("/");
					SharePathBuilder.Append(Share.Id);
					return SharePathBuilder.ToString();
				}
			}
		}
	}
	return {};
}

static FString GetAsSubPath(const FString& WorkspaceRootPath, const FString& WorkspaceSharePath)
{
	FString TestRoot = FPaths::GetPath(WorkspaceSharePath);
	while (TestRoot.Len() >= WorkspaceRootPath.Len())
	{
		if (FPaths::IsSamePath(TestRoot, WorkspaceRootPath))
		{
			return WorkspaceSharePath.Mid(TestRoot.Len() + 1);
		}
		TestRoot = FPaths::GetPath(TestRoot);
	}
	return {};
}

static FAnsiString CreateWorkspaceShare(FStorageServerConnection& QueryConnection, const FString& WorkspaceSharePath, const FStorageServerConnection::Workspaces& Workspaces)
{
	const bool IsRelative = FPaths::IsRelative(WorkspaceSharePath);
	for (const auto& Workspace : Workspaces.Workspaces)
	{
		if (Workspace.AllowShareCreationFromHttp)
		{
			FString WorkspaceRoot = Workspace.Root;
			FPaths::NormalizeDirectoryName(WorkspaceRoot);
			if (!Workspace.Root.IsEmpty())
			{
				FString SharePath = IsRelative ? WorkspaceSharePath : GetAsSubPath(WorkspaceRoot, WorkspaceSharePath);
				FPaths::NormalizeDirectoryName(SharePath);
				if (!SharePath.IsEmpty())
				{
					TIoStatusOr<FString> CreateResult = QueryConnection.CreateShare(Workspace.Id, SharePath, "");
					if (CreateResult.IsOk())
					{
						TAnsiStringBuilder<256> SharePathBuilder;
						SharePathBuilder.Append("/ws/");
						SharePathBuilder.Append(Workspace.Id);
						SharePathBuilder.Append("/");
						SharePathBuilder.Append(CreateResult.ValueOrDie());
						return SharePathBuilder.ToString();
					}
				}
			}
		}
	}
	return {};
}

FAnsiString FStorageServerPlatformFile::MakeBaseURI()
{
	TAnsiStringBuilder<256> BaseURIBuilder;
	if (UE::IsUsingZenPakFileStreaming() && !WorkspaceSharePath.IsEmpty())
	{
		FPaths::NormalizeDirectoryName(WorkspaceSharePath);
		FStorageServerConnection QueryConnection;
		if (QueryConnection.Initialize(HostAddrs, HostPort, "/ws"))
		{
			TIoStatusOr<FStorageServerConnection::Workspaces> WorkspacesResponse = QueryConnection.GetWorkspaces();
			if (WorkspacesResponse.IsOk())
			{
				const FStorageServerConnection::Workspaces& Workspaces = WorkspacesResponse.ValueOrDie();

				if (FAnsiString ExistingShareURI = FindWorkspaceSharePath(WorkspaceSharePath, Workspaces); !ExistingShareURI.IsEmpty())
				{
					return ExistingShareURI;
				}
				if (FAnsiString NewShareURI = CreateWorkspaceShare(QueryConnection, WorkspaceSharePath, Workspaces); !NewShareURI.IsEmpty())
				{
					return NewShareURI;
				}
				UE_LOGF(LogStorageServerPlatformFile, Error, "Failed to to resolve or create workspace share path '%ls' from %ls", *WorkspaceSharePath, *FString(QueryConnection.GetHostAddr()));
			}
			else
			{
				UE_LOGF(LogStorageServerPlatformFile, Error, "Failed to get list of workspaces from %ls. Status: %ls", *FString(QueryConnection.GetHostAddr()), *WorkspacesResponse.Status().ToString());
			}
		}
		else
		{
			UE_LOGF(LogStorageServerPlatformFile, Error, "Failed to connect to %ls to get list of workspace shares", *FString(QueryConnection.GetHostAddr()));
		}
	}
	if (!BaseURI.IsEmpty())
	{
		BaseURIBuilder.Append(BaseURI);
	}
	else
	{
		BaseURIBuilder.Append("/prj/");
		if (ServerProject.IsEmpty())
		{
			BaseURIBuilder.Append(TCHAR_TO_ANSI(*FApp::GetZenStoreProjectId()));
		}
		else
		{
			BaseURIBuilder.Append(ServerProject);
		}
		BaseURIBuilder.Append("/oplog/");
		if (ServerPlatform.IsEmpty())
		{
			TArray<FString> TargetPlatformNames;
			FPlatformMisc::GetValidTargetPlatforms(TargetPlatformNames);
			check(TargetPlatformNames.Num() > 0);
			BaseURIBuilder.Append(TCHAR_TO_ANSI(*TargetPlatformNames[0]));
		}
		else
		{
			BaseURIBuilder.Append(ServerPlatform);
		}
	}
	return BaseURIBuilder.ToString();
}


bool FStorageServerPlatformFile::ShouldBeUsed(IPlatformFile* Inner, const TCHAR* CmdLine) const
{
	if (FParse::Param(FCommandLine::Get(), TEXT("SkipZenStore")))
		return false;

	bool bPreferFileSystem = false;
	TArray<FString> HostNames;

#if WITH_COTF
	UE::Cook::ICookOnTheFlyModule& CookOnTheFlyModule = FModuleManager::LoadModuleChecked<UE::Cook::ICookOnTheFlyModule>(TEXT("CookOnTheFly"));
	TSharedPtr<UE::Cook::ICookOnTheFlyServerConnection> DefaultConnection = CookOnTheFlyModule.GetDefaultServerConnection();
	if (DefaultConnection.IsValid() && !DefaultConnection->GetZenProjectName().IsEmpty())
	{
		HostAddrs.Append(DefaultConnection->GetZenHostNames());
		HostPort = DefaultConnection->GetZenHostPort();
		return true;
	}
#endif
	TUniquePtr<FArchive> ProjectStoreMarkerReader = TryFindProjectStoreMarkerFile(Inner);
	if (ProjectStoreMarkerReader != nullptr)
	{
		TSharedPtr<FJsonObject> ProjectStoreObject;
		TSharedRef<TJsonReader<UTF8CHAR>> Reader = TJsonReaderFactory<UTF8CHAR>::Create(ProjectStoreMarkerReader.Get());
		if (FJsonSerializer::Deserialize(Reader, ProjectStoreObject) && ProjectStoreObject.IsValid())
		{
			const TSharedPtr<FJsonObject>* ZenServerObjectPtr = nullptr;
			if (ProjectStoreObject->TryGetObjectField(TEXT("zenserver"), ZenServerObjectPtr) && (ZenServerObjectPtr != nullptr))
			{
				const TSharedPtr<FJsonObject>& ZenServerObject = *ZenServerObjectPtr;

				FString FilesystemOperatingMode;
				if (ZenServerObject->TryGetStringField(TEXT("operatingmode"), FilesystemOperatingMode) && (FilesystemOperatingMode == "Filesystem"))
				{
					bPreferFileSystem = true;
				}

#if PLATFORM_DESKTOP || PLATFORM_ANDROID
				FString HostName;
				if (ZenServerObject->TryGetStringField(TEXT("hostname"), HostName) && !HostName.IsEmpty())
				{
					HostAddrs.Add(HostName);
				}
#endif
				const TArray<TSharedPtr<FJsonValue>>* RemoteHostNamesArrayPtr = nullptr;
				if (ZenServerObject->TryGetArrayField(TEXT("remotehostnames"), RemoteHostNamesArrayPtr) && (RemoteHostNamesArrayPtr != nullptr))
				{
					for (TSharedPtr<FJsonValue> RemoteHostName : *RemoteHostNamesArrayPtr)
					{
						if (FString RemoteHostNameStr = RemoteHostName->AsString(); !RemoteHostNameStr.IsEmpty())
						{
#if PLATFORM_IOS
							if (RemoteHostNameStr.StartsWith("macserver://"))
							{
								FString MacHostName = RemoteHostNameStr.RightChop(12);
								if (!MacHostName.IsEmpty())
								{
									// As this is the fastest connection when on USB-C, set this as the first to test
									// TODO: what about when using USB2?? Should we detect device type?
									HostAddrs.Insert(MacHostName + TEXT(".local"), 0);

									// Some macs drop the ".local", so try that as well, but as a last resort
									HostAddrs.Add(MacHostName);
								}
							}
							else
#endif 
							if (RemoteHostNameStr.StartsWith("hostname://"))
							{
								HostNames.Add(RemoteHostNameStr);
							}
							else
							{
								HostAddrs.Add(RemoteHostNameStr);
							}
						}
					}
				}

				uint16 SerializedHostPort = 0;
				if (ZenServerObject->TryGetNumberField(TEXT("hostport"), SerializedHostPort) && (SerializedHostPort != 0))
				{
					HostPort = SerializedHostPort;
				}
				UE_LOGF(LogStorageServerPlatformFile, Display, "Using connection settings from ue.projectstore: HostAddrs='%ls' and HostPort='%d'", *FString::Join(HostAddrs, TEXT("+")), HostPort);
			}
		}
		else
		{
			UE_LOGF(LogStorageServerPlatformFile, Error, "Failed to Deserialize ue.projectstore!'");
		}
	}

	FString Host;
	if (FParse::Value(CmdLine, TEXT("-ZenStoreHost="), Host))
	{
		UE_LOGF(LogStorageServerPlatformFile, Display, "Adding connection settings from command line: -ZenStoreHost='%ls'", *Host);
		if (!Host.ParseIntoArray(HostAddrs, TEXT("+"), true))
		{
			HostAddrs.Add(Host);
		}
	}
	if (FParse::Value(CmdLine, TEXT("-ZenStorePort="), HostPort))
	{
		UE_LOGF(LogStorageServerPlatformFile, Display, "Using connection settings from command line: -ZenStorePort='%d'", HostPort);
	}
	// add hostnames as last resort
	HostAddrs.Append(HostNames);

	if (!bPreferFileSystem || UE::IsUsingZenPakFileStreaming())
	{
		return HostAddrs.Num() > 0;
	}

	return false;
}

bool FStorageServerPlatformFile::Initialize(IPlatformFile* Inner, const TCHAR* CmdLine)
{
	// hybrid cooked editor wants to load any local files when possible, instead of any potential non-assets that were imported with a zen store
	if (IsRunningHybridCookedEditor())
	{
		GPreferLocalForNonAssets = 1;
	}

	LowerLevel = Inner;
	if (HostAddrs.Num() > 0)
	{
#if EXCLUDE_NONSERVER_UE_EXTENSIONS && !WITH_EDITOR
		// Extensions for file types that should only ever be on the server. Used to stop unnecessary access to the lower level platform file.
		ExcludedNonServerExtensions.Add(TEXT("uasset"));
		ExcludedNonServerExtensions.Add(TEXT("umap"));
		ExcludedNonServerExtensions.Add(TEXT("ubulk"));
		ExcludedNonServerExtensions.Add(TEXT("uexp"));
		ExcludedNonServerExtensions.Add(TEXT("uptnl"));
		ExcludedNonServerExtensions.Add(TEXT("ushaderbytecode"));
		ExcludedNonServerExtensions.Add(TEXT("ini")); //special cases of local only ini file needs to be managed as special exclusion
#endif

#if !WITH_EDITOR
		// Extensions for file types that will be assumed to be immutable - their time stamp will remain unchanged.
		AssumedImmutableTimeStampExtensions.Add(TEXT("uplugin"));

		// Extensions for file types that will be precached on startup to improve engine initialization time
		EngineStartupPrecacheExtensions.Add(TEXT("uplugin"));
		EngineStartupPrecacheExtensions.Add(TEXT("uproject"));
		EngineStartupPrecacheExtensions.Add(TEXT("ini"));
#endif

		// Don't initialize the connection yet because we want to incorporate project file path information into the initialization.

		TUniquePtr<FArchive> ProjectStoreMarkerReader = TryFindProjectStoreMarkerFile(Inner);
		if (ProjectStoreMarkerReader != nullptr)
		{
			TSharedPtr<FJsonObject> ProjectStoreObject;
			TSharedRef<TJsonReader<UTF8CHAR>> Reader = TJsonReaderFactory<UTF8CHAR>::Create(ProjectStoreMarkerReader.Get());
			if (FJsonSerializer::Deserialize(Reader, ProjectStoreObject) && ProjectStoreObject.IsValid())
			{
				const TSharedPtr<FJsonObject>* ZenServerObjectPtr = nullptr;
				if (ProjectStoreObject->TryGetObjectField(TEXT("zenserver"), ZenServerObjectPtr) && (ZenServerObjectPtr != nullptr))
				{
					const TSharedPtr<FJsonObject>& ZenServerObject = *ZenServerObjectPtr;
					ServerProject = ZenServerObject->GetStringField(TEXT("projectid"));
					ServerPlatform = ZenServerObject->GetStringField(TEXT("oplogid"));
					if (!ZenServerObject->TryGetStringField(TEXT("baseuri"), BaseURI))
					{
						BaseURI.Empty();
					}
					if (!ZenServerObject->TryGetStringField(TEXT("workspacesharepath"), WorkspaceSharePath))
					{
						WorkspaceSharePath.Empty();
					}

					const TSharedPtr<FJsonObject>* HostAuthObjectPtr = nullptr;
					if (ZenServerObject->TryGetObjectField(TEXT("hostauth"), HostAuthObjectPtr) && HostAuthObjectPtr != nullptr)
					{
						const TSharedPtr<FJsonObject>& HostAuthObject = *HostAuthObjectPtr;
						FString AuthType;
						if (HostAuthObject->TryGetStringField(TEXT("type"), AuthType))
						{
							if (AuthType == TEXT("password"))
							{
								FString Username;
								FString Password;
								HostAuthObject->TryGetStringField(TEXT("username"), Username);
								HostAuthObject->TryGetStringField(TEXT("password"), Password);
								FString Credentials = FString::Printf(TEXT("%s:%s"), *Username, *Password);
								FString Encoded = FBase64::Encode(Credentials);
								AuthHeaders.Add(FAnsiString::Printf("Authorization: Basic %s", TCHAR_TO_ANSI(*Encoded)));
								UE_LOGF(LogStorageServerPlatformFile, Display, "Using password authentication from ue.projectstore");
							}
						}
					}

					UE_LOGF(LogStorageServerPlatformFile, Display, "Using settings from ue.projectstore: ServerProject='%ls' and ServerPlatform='%ls'", *ServerProject, *ServerPlatform);
				}

				const TArray<TSharedPtr<FJsonValue>>* RemapDirectoriesArrayPtr = nullptr;
				if ( ProjectStoreObject->TryGetArrayField(TEXT("remapDirectories"), RemapDirectoriesArrayPtr) )
				{
					for (const TSharedPtr<FJsonValue>& JsonValue : *RemapDirectoriesArrayPtr)
					{
						const TSharedPtr<FJsonObject>& RemapObject = JsonValue->AsObject();
						const FString RemapFrom = RemapObject->GetStringField(TEXT("from"));
						const FString RemapTo   = RemapObject->GetStringField(TEXT("to"));
						RemapDirectoriesTree.FindOrAdd(RemapFrom) = RemapTo;
					}
					RemapDirectoriesTree.Shrink();
				}
			}
		}
	
		if (FParse::Value(CmdLine, TEXT("-ZenStoreProject="), ServerProject))
		{
			UE_LOGF(LogStorageServerPlatformFile, Display, "Using settings from command line: -ZenStoreProject='%ls'", *ServerProject);
		}
		if (FParse::Value(CmdLine, TEXT("-ZenStorePlatform="), ServerPlatform))
		{
			UE_LOGF(LogStorageServerPlatformFile, Display, "Using settings from command line: -ZenStorePlatform='%ls'", *ServerPlatform);
		}
		if (FParse::Value(CmdLine, TEXT("-ZenStoreBaseURI="), BaseURI))
		{
			UE_LOGF(LogStorageServerPlatformFile, Display, "Using settings from command line: -ZenStoreBaseURI='%ls'", *BaseURI);
		}
		if (FParse::Value(CmdLine, TEXT("-ZenWorkspaceSharePath="), WorkspaceSharePath))
		{
			UE_LOGF(LogStorageServerPlatformFile, Display, "Using settings from command line: -ZenWorkspaceSharePath='%ls'", *WorkspaceSharePath);
		}

		if (UE::IsUsingZenPakFileStreaming())
		{
			InitializeConnection();
		}

		return true;
	}
	return false;
}

void FStorageServerPlatformFile::InitializeAfterProjectFilePath()
{
	AbsEngineDir = FPaths::ConvertRelativePathToFull(FPlatformMisc::EngineDir());
	AbsProjectDir = FPaths::ConvertRelativePathToFull(FPlatformMisc::ProjectDir());

	InitializeConnection();

	// optional debugging module depends on a valid Connection
	if (FModuleManager::Get().ModuleExists(TEXT("StorageServerClientDebug")))
	{
		FModuleManager::Get().LoadModule("StorageServerClientDebug");
	}
}

void FStorageServerPlatformFile::InitializeConnection()
{
	if (Connection)
	{
		return;
	}

#if WITH_COTF
	UE::Cook::ICookOnTheFlyModule& CookOnTheFlyModule = FModuleManager::LoadModuleChecked<UE::Cook::ICookOnTheFlyModule>(TEXT("CookOnTheFly"));
	CookOnTheFlyServerConnection = CookOnTheFlyModule.GetDefaultServerConnection();
	if (CookOnTheFlyServerConnection)
	{
		CookOnTheFlyServerConnection->OnMessage().AddRaw(this, &FStorageServerPlatformFile::OnCookOnTheFlyMessage);
		ServerProject = CookOnTheFlyServerConnection->GetZenProjectName();
		ServerPlatform = CookOnTheFlyServerConnection->GetPlatformName();
	}
#endif
	Connection.Reset(new FStorageServerConnection());
	if (Connection->Initialize(HostAddrs, HostPort, MakeBaseURI(), AuthHeaders))
	{
#if WITH_STORAGE_SERVER_STARTUP_FILE_CACHE
		GStorageServerEngineStartupPrecache.Reset( new FStorageServerEngineStartupPrecache(*Connection.Get()) );
		FCoreDelegates::OnFEngineLoopInitComplete.AddLambda( []()
		{
			GStorageServerEngineStartupPrecache.Reset();
		});	
#endif

		if (SendGetFileListMessage())
		{
			if (bAllowPackageIo)
			{
				FIoDispatcher& IoDispatcher = FIoDispatcher::Get();
				TSharedRef<FStorageServerIoDispatcherBackend> IoDispatcherBackend = MakeShared<FStorageServerIoDispatcherBackend>(*Connection.Get());
				IoDispatcher.Mount(IoDispatcherBackend);
#if WITH_COTF
				if (CookOnTheFlyServerConnection)
				{
					FPackageStore::Get().Mount(MakeShared<FCookOnTheFlyPackageStoreBackend>(*CookOnTheFlyServerConnection.Get()));
				}
				else
#endif
				{
					FPackageStore::Get().Mount(MakeShared<FStorageServerPackageStoreBackend>(*Connection.Get()));
				}
			}
		}
		else
		{
			FStringView HostAddr = Connection->GetHostAddr();
			UE_LOGF(LogStorageServerPlatformFile, Fatal, "Failed to get file list from Zen at '%.*ls'", HostAddr.Len(), HostAddr.GetData());
		}
	}
	else if (bAbortOnConnectionFailure)
	{
		if (!FApp::IsUnattended())
		{
			FString FailedConnectionTitle = "Failed to connect";
			FString FailedConnectionText = FString::Printf(TEXT(
				"Network data streaming failed to connect to any of the following data sources:\n\n%s\n\n"
				"This can be due to the sources being offline, the Unreal Zen Storage process not currently running, "
				"invalid addresses, firewall blocking, or the sources being on a different network from this device.\n"
				"Please verify that your Unreal Zen Storage process is running using the ZenDashboard utility, "
				"and ue.projectstore file in the staged folder contains the valid IP address of the host PC in the \"remotehostnames\" section.\n"
				"If these issues can't be addressed, you can use an installed build without network data streaming by "
				"building with the '-pak' argument. This process will now exit."),
				*FString::Join(HostAddrs, TEXT("\n")));
			FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *FailedConnectionText, *FailedConnectionTitle);
		}

		UE_LOGF(LogStorageServerPlatformFile, Error, "Failed to initialize connection to any of [ %ls ]",
			*FString::Join(HostAddrs, TEXT(", ")));
		Connection.Reset();
		FPlatformMisc::RequestExit(true);
	}
	else
	{
		// One reason the connection can fail is if the oplog has not yet been created during a new cook:
		// we will get the message "oplog <PlatformName> not found in project <ProjectName>".
		UE_LOGF(LogStorageServerPlatformFile, Display, "Failed to initialize connection to any of [ %ls ]",
			*FString::Join(HostAddrs, TEXT(", ")));
		Connection.Reset();
	}
}

void FStorageServerPlatformFile::UpdateFileList()
{
	if (!Connection)
	{
		return;
	}

	ServerToc.Clear();
	if (!SendGetFileListMessage())
	{
		FStringView HostAddr = Connection->GetHostAddr();
		UE_LOGF(LogStorageServerPlatformFile, Fatal, "Failed to get file list from Zen at '%.*ls'", HostAddr.Len(), HostAddr.GetData());
	}
}

TSharedPtr<IPackageStoreBackend> FStorageServerPlatformFile::CreatePackageStoreBackend() const
{
#if WITH_COTF
	if (CookOnTheFlyServerConnection)
	{
		return MakeShared<FCookOnTheFlyPackageStoreBackend>(*CookOnTheFlyServerConnection.Get());
	}
	else
#endif
	{
		return MakeShared<FStorageServerPackageStoreBackend>(*Connection.Get());
	}
}

TSharedPtr<IIoDispatcherBackend> FStorageServerPlatformFile::CreateIoDispatcherBackend() const
{
	return MakeShared<FStorageServerIoDispatcherBackend>(*Connection.Get());
}

bool FStorageServerPlatformFile::FileExists(const TCHAR* Filename)
{
	TStringBuilder<1024> StorageServerFilename;
	if (MakeStorageServerPath(Filename, StorageServerFilename) && ServerToc.FileExists(*StorageServerFilename))
	{
		return true;
	}

	return (LowerLevel && IsNonServerFilenameAllowed(Filename)) ? LowerLevel->FileExists(Filename) : false;
}

FDateTime FStorageServerPlatformFile::GetTimeStamp(const TCHAR* Filename)
{
	TStringBuilder<1024> StorageServerFilename;
	if (MakeStorageServerPath(Filename, StorageServerFilename))
	{
		if (ServerToc.FileExists(*StorageServerFilename))
		{
			return IsAssumedImmutableTimeStampFilename(*StorageServerFilename) ? GAssumedImmutableTimeStamp : FDateTime::Now();
		}
	}
	return (LowerLevel && IsNonServerFilenameAllowed(Filename)) ? LowerLevel->GetTimeStamp(Filename) : FDateTime::MinValue();
}

FDateTime FStorageServerPlatformFile::GetAccessTimeStamp(const TCHAR* Filename)
{
	TStringBuilder<1024> StorageServerFilename;
	if (MakeStorageServerPath(Filename, StorageServerFilename))
	{
		if (ServerToc.FileExists(*StorageServerFilename))
		{
			return IsAssumedImmutableTimeStampFilename(*StorageServerFilename) ? GAssumedImmutableTimeStamp : FDateTime::Now();
		}
	}
	return (LowerLevel && IsNonServerFilenameAllowed(Filename)) ? LowerLevel->GetAccessTimeStamp(Filename) : FDateTime::MinValue();
}

int64 FStorageServerPlatformFile::FileSize(const TCHAR* Filename)
{
	TStringBuilder<1024> StorageServerFilename;
	if (MakeStorageServerPath(Filename, StorageServerFilename))
	{
		int64 FileSize = ServerToc.GetFileSize(*StorageServerFilename);
		if (FileSize > STORAGE_SERVER_FILE_UNKOWN_SIZE)
		{
			return FileSize;
		}
	}
	return (LowerLevel && IsNonServerFilenameAllowed(Filename)) ? LowerLevel->FileSize(Filename) : STORAGE_SERVER_FILE_UNKOWN_SIZE;
}

bool FStorageServerPlatformFile::IsReadOnly(const TCHAR* Filename)
{
	TStringBuilder<1024> StorageServerFilename;
	if (MakeStorageServerPath(Filename, StorageServerFilename) && ServerToc.FileExists(*StorageServerFilename))
	{
		return true;
	}
	return (LowerLevel && IsNonServerFilenameAllowed(Filename)) ? LowerLevel->IsReadOnly(Filename) : false;
}

FFileStatData FStorageServerPlatformFile::GetStatData(const TCHAR* FilenameOrDirectory)
{
	TStringBuilder<1024> StorageServerFilenameOrDirectory;
	if (MakeStorageServerPath(FilenameOrDirectory, StorageServerFilenameOrDirectory))
	{
		int64 FileSize = ServerToc.GetFileSize(*StorageServerFilenameOrDirectory);
		if (FileSize > STORAGE_SERVER_FILE_UNKOWN_SIZE)
		{
			return FFileStatData(
				FDateTime::Now(),
				FDateTime::Now(),
				FDateTime::Now(),
				FileSize,
				false,
				true);
		}
		else if (ServerToc.DirectoryExists(*StorageServerFilenameOrDirectory))
		{
			return FFileStatData(
				FDateTime::MinValue(),
				FDateTime::MinValue(),
				FDateTime::MinValue(),
				0,
				true,
				true);
		}
	}
	FFileStatData FileStatData;
	if (LowerLevel && IsNonServerFilenameAllowed(FilenameOrDirectory))
	{
		FileStatData = LowerLevel->GetStatData(FilenameOrDirectory);
	}
	return FileStatData;
}

IFileHandle* FStorageServerPlatformFile::InternalOpenFile(const FIoChunkId& FileChunkId, int64 RawSize, const TCHAR* LocalFilename)
{
	IFileHandle* FileHandle = new FStorageServerFileHandle(*this, FileChunkId, RawSize, LocalFilename);
	IWrappedFileHandle* FileDecompressor = CreateCompressedPlatformFileHandle(FileHandle);
	
	return FileDecompressor ? FileDecompressor : FileHandle;
}

IFileHandle* FStorageServerPlatformFile::OpenRead(const TCHAR* Filename, bool bAllowWrite)
{
	TStringBuilder<1024> StorageServerFilename;

	// if we prefer local files, look local before checking if the file is in the ServerToc
	if (GPreferLocalForNonAssets && LowerLevel && IsNonServerFilenameAllowed(Filename))
	{
		IFileHandle* Handle = LowerLevel->OpenRead(Filename, bAllowWrite);
		if (Handle != nullptr)
		{
			return Handle;
		}
	}	

	if (MakeStorageServerPath(Filename, StorageServerFilename))
	{
		FIoChunkId FileChunkId;
		int64 RawSize = STORAGE_SERVER_FILE_UNKOWN_SIZE;
		if (ServerToc.GetFileData(*StorageServerFilename, FileChunkId, RawSize))
		{
			return InternalOpenFile(FileChunkId, RawSize, Filename);
		}
	}
	// if we preferred Server over Local, look in local if Server failed
	return (!GPreferLocalForNonAssets && LowerLevel && IsNonServerFilenameAllowed(Filename)) ? LowerLevel->OpenRead(Filename, bAllowWrite) : nullptr;
}


// a reusable class that handles iterating two different locations that will return results that appear
// to the engine as one location (in this case the StorageServer and Local Files will have the same path for UE)
// and we need to only return 1 copy, otherwise we can cause errors ot duplicated work)
template<typename ParentVisitorClass, typename DataType>
class FUniqueDirectoryStatVisitor : public ParentVisitorClass
{
private:
	TSet<FString> AlreadyVisited;
	ParentVisitorClass& RealVisitor;
	IPlatformFile* LowerLevel;
	FStorageServerPlatformFile& ServerPlatformFile;
	bool bKeepIterating = true;

public:
	FUniqueDirectoryStatVisitor(FStorageServerPlatformFile* InPlatformFile, ParentVisitorClass& InVisitor)
		: RealVisitor(InVisitor)
		, LowerLevel(InPlatformFile->GetLowerLevel())
		, ServerPlatformFile(*InPlatformFile)
	{
	}

	virtual bool ShouldVisitLeafPathname(FStringView LeafPathname) override
	{
		return RealVisitor.ShouldVisitLeafPathname(LeafPathname);
	}

	virtual bool Visit(const TCHAR* FilenameOrDirectory, DataType StatData) override
	{
		// for speed reasons, we only do the double location checks when we enable the new
		// PreferLocal behavior - if we ever have duped results without it, we can remove 
		// this check
		if (GPreferLocalForNonAssets)
		{
			FString FileStr = FPaths::ConvertRelativePathToFull(FilenameOrDirectory);

			if (AlreadyVisited.Contains(FileStr))
			{
				return true;
			}
			AlreadyVisited.Add(FileStr);
		}
		bKeepIterating = bKeepIterating && RealVisitor.Visit(FilenameOrDirectory, StatData);
		return bKeepIterating;
	}

	bool PerformMergedIteration(const TCHAR* Directory,
		TFunctionRef<bool(const TCHAR* Directory, ParentVisitorClass& Visitor)> LowLevelOperation,
		TFunctionRef<bool(const TCHAR* Directory, ParentVisitorClass& Visitor)> RemoteOperation)
	{
		// first, if we prefer local assets, iterate on local first (in case the data doesn't match remote for whatever reason,
		// local data will be used)
		bool bLocalDirectoryExists = false;
		bool bRemoteDirectoryExists = false;
		if (GPreferLocalForNonAssets)
		{
			bool bExistsAndKeepIterating = LowLevelOperation(Directory, *this);
			if (!bKeepIterating)
			{
				return false;
			}
			bLocalDirectoryExists = bExistsAndKeepIterating;
		}

		// then look remote
		TStringBuilder<1024> StorageServerDirectory;
		bool bShouldContinue = true;
		bool bIterateOnServer = ServerPlatformFile.MakeStorageServerPath(Directory, StorageServerDirectory) && ServerPlatformFile.ServerToc.DirectoryExists(*StorageServerDirectory);
		if (bIterateOnServer)
		{
			bool bExistsAndKeepIterating = RemoteOperation(*StorageServerDirectory, *this);
			if (!bKeepIterating)
			{
				return false;
			}
			bRemoteDirectoryExists = bExistsAndKeepIterating;
		}

		// finall look locally if we are preferring remote over local
		if (!GPreferLocalForNonAssets)
		{
			bool bExistsAndKeepIterating = LowLevelOperation(Directory, *this);
			if (!bKeepIterating)
			{
				return false;
			}
			bLocalDirectoryExists = bExistsAndKeepIterating;
		}

		return bKeepIterating && (bLocalDirectoryExists || bRemoteDirectoryExists);
	}

};


bool FStorageServerPlatformFile::IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor)
{
	FUniqueDirectoryStatVisitor<FDirectoryVisitor, bool> MergedVisitor(this, Visitor);

	return MergedVisitor.PerformMergedIteration(Directory,
		[this](const TCHAR* Directory, FDirectoryVisitor& InMergedVisitor)
		{
			return LowerLevel ? LowerLevel->IterateDirectory(Directory, InMergedVisitor) : false;
		},
		[this](const TCHAR* Directory, FDirectoryVisitor& InMergedVisitor)
		{
			return ServerToc.IterateDirectory(Directory, [this, &InMergedVisitor]
				(const FIoChunkId& FileChunkId, const TCHAR* FilenameOrDirectory, int64 RawSize)
				{
					TStringBuilder<1024> LocalPath;
					bool bConverted = MakeLocalPath(FilenameOrDirectory, LocalPath);
					check(bConverted);
					const bool bDirectory = !FileChunkId.IsValid();
					return InMergedVisitor.CallShouldVisitAndVisit(*LocalPath, bDirectory);
				});
		}
	);
}

bool FStorageServerPlatformFile::IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor)
{
	FUniqueDirectoryStatVisitor<FDirectoryVisitor, bool> MergedVisitor(this, Visitor);

	return MergedVisitor.PerformMergedIteration(Directory,
		[this](const TCHAR* Directory, FDirectoryVisitor& InMergedVisitor)
		{
			return LowerLevel ? LowerLevel->IterateDirectoryRecursively(Directory, InMergedVisitor) : false;
		},
		[this](const TCHAR* Directory, FDirectoryVisitor& InMergedVisitor)
		{
			return ServerToc.IterateDirectoryRecursively(Directory, [this, &InMergedVisitor]
				(const FIoChunkId& FileChunkId, const TCHAR* FilenameOrDirectory, int64 RawSize)
				{
					TStringBuilder<1024> LocalPath;
					bool bConverted = MakeLocalPath(FilenameOrDirectory, LocalPath);
					check(bConverted);
					const bool bDirectory = !FileChunkId.IsValid();
					return InMergedVisitor.CallShouldVisitAndVisit(*LocalPath, bDirectory);
				});
		}
	);
}

bool FStorageServerPlatformFile::IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor)
{
	FUniqueDirectoryStatVisitor<FDirectoryStatVisitor, const FFileStatData&> MergedVisitor(this, Visitor);

	return MergedVisitor.PerformMergedIteration(Directory,
		[this](const TCHAR* Directory, FDirectoryStatVisitor& InMergedVisitor)
		{
			return LowerLevel ? LowerLevel->IterateDirectoryStat(Directory, InMergedVisitor) : false;
		},
		[this](const TCHAR* Directory, FDirectoryStatVisitor& InMergedVisitor)
		{
			return ServerToc.IterateDirectory(Directory, [this, &InMergedVisitor]
				(const FIoChunkId& FileChunkId, const TCHAR* ServerFilenameOrDirectory, int64 RawSize)
				{
					TStringBuilder<1024> LocalPath;
					bool bConverted = MakeLocalPath(ServerFilenameOrDirectory, LocalPath);
					check(bConverted);
					FFileStatData FileStatData;
					if (FileChunkId.IsValid())
					{
						FileStatData = FFileStatData(
							FDateTime::Now(),
							FDateTime::Now(),
							FDateTime::Now(),
							RawSize,
							false,
							true);
						check(FileStatData.bIsValid);
					}
					else
					{
						FileStatData = FFileStatData(
							FDateTime::MinValue(),
							FDateTime::MinValue(),
							FDateTime::MinValue(),
							0,
							true,
							true);
					}
					return InMergedVisitor.CallShouldVisitAndVisit(*LocalPath, FileStatData);
				});
		}
	);
}

FOpenMappedResult FStorageServerPlatformFile::OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions, int64 MaximumSize)
{
	if (LowerLevel && IsNonServerFilenameAllowed(Filename))
	{
		return LowerLevel->OpenMappedEx(Filename, OpenOptions, MaximumSize);
	}
	return MakeError(FString::Printf(TEXT("Can't open mapped file '%s'"), Filename));
}

FOpenMappedResult FStorageServerPlatformFile::OpenMappedEx2(const TCHAR* Filename, EOpenMappedFlags OpenOptions, int64 MaximumSize)
{
	if (LowerLevel && IsNonServerFilenameAllowed(Filename))
	{
		return LowerLevel->OpenMappedEx2(Filename, OpenOptions, MaximumSize);
	}
	return MakeError(FString::Printf(TEXT("Can't open mapped file '%s'"), Filename));
}

bool FStorageServerPlatformFile::DirectoryExists(const TCHAR* Directory)
{
	TStringBuilder<1024> StorageServerDirectory;
	if (MakeStorageServerPath(Directory, StorageServerDirectory) && ServerToc.DirectoryExists(*StorageServerDirectory))
	{
		return true;
	}
	return LowerLevel && LowerLevel->DirectoryExists(Directory);
}

FString FStorageServerPlatformFile::GetFilenameOnDisk(const TCHAR* Filename)
{
	TStringBuilder<1024> StorageServerFilename;
	if (MakeStorageServerPath(Filename, StorageServerFilename) && ServerToc.FileExists(*StorageServerFilename))
	{
		UE_LOGF(LogStorageServerPlatformFile, Warning, "Attempting to get disk filename of remote file '%ls'", Filename);
		return Filename;
	}
	return (LowerLevel && IsNonServerFilenameAllowed(Filename)) ? LowerLevel->GetFilenameOnDisk(Filename) : Filename;
}

bool FStorageServerPlatformFile::DeleteFile(const TCHAR* Filename)
{
	// if we prefer local files, we can delete them (without this, if the file is in the ServerToc, it will fail to delete)
	if (GPreferLocalForNonAssets && LowerLevel && LowerLevel->DeleteFile(Filename))
	{
		return true;
	}

	TStringBuilder<1024> StorageServerFilename;
	if (MakeStorageServerPath(Filename, StorageServerFilename) && ServerToc.FileExists(*StorageServerFilename))
	{
		return false;
	}
	return LowerLevel && LowerLevel->DeleteFile(Filename);
}

bool FStorageServerPlatformFile::MoveFile(const TCHAR* To, const TCHAR* From)
{
	if (!LowerLevel)
	{
		return false;
	}

	if (GPreferLocalForNonAssets && LowerLevel->MoveFile(To, From))
	{
		return true;
	}

	TStringBuilder<1024> StorageServerTo;
	if (MakeStorageServerPath(To, StorageServerTo) && ServerToc.FileExists(*StorageServerTo))
	{
		return false;
	}
	TStringBuilder<1024> StorageServerFrom;
	if (MakeStorageServerPath(From, StorageServerFrom))
	{
		FIoChunkId FromFileChunkId;
		int64 FromFileRawSize = STORAGE_SERVER_FILE_UNKOWN_SIZE;
		if (ServerToc.GetFileData(*StorageServerFrom, FromFileChunkId, FromFileRawSize))
		{
			TUniquePtr<IFileHandle> ToFile(LowerLevel->OpenWrite(To, false, false));
			if (!ToFile)
			{
				return false;
			}

			TUniquePtr<IFileHandle> FromFile(InternalOpenFile(FromFileChunkId, FromFileRawSize, *StorageServerFrom));
			if (!FromFile)
			{
				return false;
			}
			const int64 BufferSize = 64 << 10;
			TArray<uint8> Buffer;
			Buffer.SetNum(BufferSize);
			int64 BytesLeft = FromFile->Size();
			while (BytesLeft)
			{
				int64 BytesToWrite = FMath::Min(BufferSize, BytesLeft);
				if (!FromFile->Read(Buffer.GetData(), BytesToWrite))
				{
					return false;
				}
				if (!ToFile->Write(Buffer.GetData(), BytesToWrite))
				{
					return false;
				}
				BytesLeft -= BytesToWrite;
			}
			return true;
		}
	}
	return LowerLevel->MoveFile(To, From);
}

bool FStorageServerPlatformFile::SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue)
{
	TStringBuilder<1024> StorageServerFilename;
	if (MakeStorageServerPath(Filename, StorageServerFilename) && ServerToc.FileExists(*StorageServerFilename))
	{
		return bNewReadOnlyValue;
	}
	return LowerLevel && LowerLevel->SetReadOnly(Filename, bNewReadOnlyValue);
}

void FStorageServerPlatformFile::SetTimeStamp(const TCHAR* Filename, FDateTime DateTime)
{
	TStringBuilder<1024> StorageServerFilename;
	if (MakeStorageServerPath(Filename, StorageServerFilename) && ServerToc.FileExists(*StorageServerFilename))
	{
		return;
	}
	if (LowerLevel)
	{
		LowerLevel->SetTimeStamp(Filename, DateTime);
	}
}

IFileHandle* FStorageServerPlatformFile::OpenWrite(const TCHAR* Filename, bool bAppend, bool bAllowRead)
{
	TStringBuilder<1024> StorageServerFilename;
	if (MakeStorageServerPath(Filename, StorageServerFilename) && ServerToc.FileExists(*StorageServerFilename))
	{
		return nullptr;
	}
	if (LowerLevel)
	{
		return LowerLevel->OpenWrite(Filename, bAppend, bAllowRead);
	}
	return nullptr;
}

bool FStorageServerPlatformFile::CreateDirectory(const TCHAR* Directory)
{
	TStringBuilder<1024> StorageServerDirectory;
	if (MakeStorageServerPath(Directory, StorageServerDirectory) && ServerToc.DirectoryExists(*StorageServerDirectory))
	{
		return true;
	}
	return LowerLevel && LowerLevel->CreateDirectory(Directory);
}

bool FStorageServerPlatformFile::DeleteDirectory(const TCHAR* Directory)
{
	TStringBuilder<1024> StorageServerDirectory;
	if (MakeStorageServerPath(Directory, StorageServerDirectory) && ServerToc.DirectoryExists(*StorageServerDirectory))
	{
		return false;
	}
	return LowerLevel && LowerLevel->DeleteDirectory(Directory);
}

FString FStorageServerPlatformFile::ConvertToAbsolutePathForExternalAppForRead(const TCHAR* Filename)
{
#if PLATFORM_DESKTOP && (UE_GAME || UE_SERVER)
	TStringBuilder<1024> Result;

	// New code should not end up in here and should instead be written in such a
	// way that data can be served from a (remote) server.

	// Some data must exist in files on disk such that it can be accessed by external
	// APIs. Any such data required by a title should have been written to Saved/Cooked
	// at cook time. If a file prefix with UE's canonical ../../../ is requested we
	// look inside Saved/Cooked. A read-only filesystem overlay if you will.

	static FString* CookedDir = nullptr;
	if (CookedDir == nullptr)
	{
		static FString Inner;
		CookedDir = &Inner;

		Result << *FPaths::ProjectDir();
		Result << TEXT("Saved/Cooked/");
		Result << FPlatformProperties::PlatformName();
		Result << TEXT("/");
		Inner = Result.ToString();
	}
	else
	{
		Result << *(*CookedDir);
	}

	const TCHAR* DotSlashSkip = Filename;
	for (; *DotSlashSkip == '.' || *DotSlashSkip == '/'; ++DotSlashSkip);

	if (PTRINT(DotSlashSkip - Filename) == 9) // 9 == ../../../
	{
		Result << DotSlashSkip;
		if (LowerLevel && LowerLevel->FileExists(Result.ToString()))
		{
			return FString::ConstructFromPtrSize(Result.GetData(), Result.Len());
		}
	}
#endif

	if (LowerLevel)
	{
		return LowerLevel->ConvertToAbsolutePathForExternalAppForRead(Filename);
	}

	return IStorageServerPlatformFile::ConvertToAbsolutePathForExternalAppForRead(Filename);
}

FString FStorageServerPlatformFile::ConvertToAbsolutePathForExternalAppForWrite(const TCHAR* Filename)
{
	if (LowerLevel)
	{
		return LowerLevel->ConvertToAbsolutePathForExternalAppForWrite(Filename);
	}

	return IStorageServerPlatformFile::ConvertToAbsolutePathForExternalAppForWrite(Filename);
}

bool FStorageServerPlatformFile::IsNonServerFilenameAllowed(FStringView InFilename)
{
	bool bAllowed = true;

#if EXCLUDE_NONSERVER_UE_EXTENSIONS
	if (!HostAddrs.IsEmpty() && (LowerLevel == &IPlatformFile::GetPlatformPhysical()))
	{
		bool bRelative = FPathViews::IsRelativePath(InFilename);

		if (bRelative)
		{
			FName Ext = FName(FPathViews::GetExtension(InFilename));
			bAllowed = !ExcludedNonServerExtensions.Contains(Ext);
			if (!bAllowed)
			{
				// There are portions of the directory tree we expect to be writable, so we should
				// allow reads to happen from them via the local filesystem, even for file extensions
				// that we otherwise consider the storage server to be authoritative for.
				if (IsFileInDynamicPath(FString(InFilename)))
				{
					bAllowed = true;
				}
			}

			UE_CLOGF(!bAllowed, LogStorageServerPlatformFile, VeryVerbose,
				"Access to file '%.*ls' is limited to server contents due to file extension being listed in ExcludedNonServerExtensions.",
				InFilename.Len(), InFilename.GetData())
		}
	}
#endif

	return bAllowed;
}

bool FStorageServerPlatformFile::IsAssumedImmutableTimeStampFilename(FStringView InFilename) const
{
	FName Ext = FName(FPathViews::GetExtension(InFilename));
	return AssumedImmutableTimeStampExtensions.Contains(Ext);
}

bool FStorageServerPlatformFile::IsEngineStartupPrecachableFilename(FStringView InFilename) const
{
	FName Ext = FName(FPathViews::GetExtension(InFilename));
	return EngineStartupPrecacheExtensions.Contains(Ext);
}

bool FStorageServerPlatformFile::MakeStorageServerPath(const TCHAR* LocalFilenameOrDirectory, FStringBuilderBase& OutPath) const
{
	FStringView LocalEngineDirView(FPlatformMisc::EngineDir());
	FStringView LocalProjectDirView(FPlatformMisc::ProjectDir());
	FStringView LocalFilenameOrDirectoryView(LocalFilenameOrDirectory);
	bool bValid = false;

	if (LocalFilenameOrDirectoryView.StartsWith(LocalEngineDirView, ESearchCase::IgnoreCase))
	{
		OutPath.Append(ServerEngineDirView);
		OutPath.Append(LocalFilenameOrDirectoryView.RightChop(LocalEngineDirView.Len()));
		bValid = true;
	}
	else if (LocalFilenameOrDirectoryView.StartsWith(LocalProjectDirView, ESearchCase::IgnoreCase))
	{
		OutPath.Append(ServerProjectDirView);
		OutPath.Append(LocalFilenameOrDirectoryView.RightChop(LocalProjectDirView.Len()));
		bValid = true;
	}
	else
	{
		TStringBuilder<128> AbsPathBuilder;
		FPathViews::ToAbsolutePath(LocalFilenameOrDirectoryView, AbsPathBuilder);
		FStringView RelativePath;
		if (FPathViews::TryMakeChildPathRelativeTo(AbsPathBuilder, AbsProjectDir, RelativePath))
		{
			OutPath.Append(ServerProjectDirView);
			OutPath.Append(RelativePath);
			bValid = true;
		}
		else if (FPathViews::TryMakeChildPathRelativeTo(AbsPathBuilder, AbsEngineDir, RelativePath))
		{
			OutPath.Append(ServerEngineDirView);
			OutPath.Append(RelativePath);
			bValid = true;
		}
	}

	if (bValid)
	{
		Algo::Replace(MakeArrayView(OutPath), '\\', '/');
		OutPath.RemoveSuffix(LocalFilenameOrDirectoryView.EndsWith('/') ? 1 : 0);
	}

	return bValid;
}

bool FStorageServerPlatformFile::MakeLocalPath(const TCHAR* ServerFilenameOrDirectory, FStringBuilderBase& OutPath) const
{
	FStringView ServerFilenameOrDirectoryView(ServerFilenameOrDirectory);
	if (ServerFilenameOrDirectoryView.StartsWith(ServerEngineDirView, ESearchCase::IgnoreCase))
	{
		OutPath.Append(FPlatformMisc::EngineDir());
		OutPath.Append(ServerFilenameOrDirectoryView.RightChop(ServerEngineDirView.Len()));
		return true;
	}
	else if (ServerFilenameOrDirectoryView.StartsWith(ServerProjectDirView, ESearchCase::IgnoreCase))
	{
		OutPath.Append(FPlatformMisc::ProjectDir());
		OutPath.Append(ServerFilenameOrDirectoryView.RightChop(ServerProjectDirView.Len()));
		return true;
	}
	return false;
}

bool FStorageServerPlatformFile::SendGetFileListMessage()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(StorageServerPlatformFileGetFileList);
	
	Connection->FileManifestRequest([&](FIoChunkId Id, FStringView Path, int64 RawSize)
	{
		FString RemapPathFrom;
		FString* RemapPathToPtr = nullptr;
		TStringBuilder<1024> RemappedPath;
		if (RemapDirectoriesTree.TryFindClosestPath(Path, RemapPathFrom, &RemapPathToPtr))
		{
			bool bTrimExtraSeparator = RemapPathFrom[RemapPathFrom.Len()-1] == TEXT('/');

			RemappedPath.Append(*RemapPathToPtr);
			RemappedPath.Append(Path.RightChop(RemapPathFrom.Len() + (bTrimExtraSeparator ? 0 : 1)));
			Path = RemappedPath;
		}

		ServerToc.AddFile(Id, Path, RawSize);

#if WITH_STORAGE_SERVER_STARTUP_FILE_CACHE
		if (RawSize > 0 && RawSize < FStorageServerEngineStartupPrecache::MaxFileSize && IsEngineStartupPrecachableFilename(Path) )
		{
			const bool bHighPriority = Path.EndsWith(TEXT(".uproject")) || Path.EndsWith(TEXT("DataDrivenPlatformInfo.ini")); // special case: we know the .uproject and DDPI will be needed immediately after this
			GStorageServerEngineStartupPrecache->AddPrecachedFile(Id, RawSize, bHighPriority);
		}
#endif
	});

#if WITH_STORAGE_SERVER_STARTUP_FILE_CACHE
	GStorageServerEngineStartupPrecache->Finalize();
#endif

	return true;
}

FFileStatData FStorageServerPlatformFile::SendGetStatDataMessage(const FIoChunkId& FileChunkId)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(StorageServerPlatformFileGetStatData);
	const int64 FileSize = Connection->ChunkSizeRequest(FileChunkId);
	if (FileSize < 0)
	{
		return FFileStatData();
	}

	FDateTime CreationTime = FDateTime::Now();
	FDateTime AccessTime = FDateTime::Now();
	FDateTime ModificationTime = FDateTime::Now();

	return FFileStatData(CreationTime, AccessTime, ModificationTime, FileSize, false, true);
}

int64 FStorageServerPlatformFile::SendReadMessage(uint8* Destination, const FIoChunkId& FileChunkId, int64 Offset, int64 BytesToRead)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(StorageServerPlatformFileRead);
	TIoStatusOr<FIoBuffer> Result = Connection->ReadChunkRequest(FileChunkId, Offset, BytesToRead, FIoBuffer(FIoBuffer::Wrap, Destination, BytesToRead), false);
	return Result.IsOk() ? Result.ValueOrDie().GetSize() : 0;
}

bool FStorageServerPlatformFile::SendMessageToServer(const TCHAR* Message, IPlatformFile::IFileServerMessageHandler* Handler)
{
#if WITH_COTF
	if (!CookOnTheFlyServerConnection->IsConnected())
	{
		return false;
	}
	if (FCString::Stricmp(Message, TEXT("RecompileShaders")) == 0)
	{
		UE::Cook::FCookOnTheFlyRequest Request(UE::Cook::ECookOnTheFlyMessage::RecompileShaders);
		{
			TUniquePtr<FArchive> Ar = Request.WriteBody();
			Handler->FillPayload(*Ar);
		}

		UE::Cook::FCookOnTheFlyResponse Response = CookOnTheFlyServerConnection->SendRequest(Request).Get();
		if (Response.IsOk())
		{
			TUniquePtr<FArchive> Ar = Response.ReadBody();
			Handler->ProcessResponse(*Ar);
		}

		return Response.IsOk();
	}
#endif
	return false;
}

FStringView FStorageServerPlatformFile::GetHostAddr() const
{
	return Connection->GetHostAddr();
}

void FStorageServerPlatformFile::GetAndResetConnectionStats(FConnectionStats& OutStats)
{
	return Connection->GetAndResetStats(OutStats);
}

#if WITH_COTF
void FStorageServerPlatformFile::OnCookOnTheFlyMessage(const UE::Cook::FCookOnTheFlyMessage& Message)
{
	switch (Message.GetHeader().MessageType)
	{
		case UE::Cook::ECookOnTheFlyMessage::FilesAdded:
		{
			UE_LOGF(LogCookOnTheFly, Verbose, "Received '%ls' message", LexToString(Message.GetHeader().MessageType));

			TArray<FString> Filenames;
			TArray<FIoChunkId> ChunkIds;

			{
				TUniquePtr<FArchive> Ar = Message.ReadBody();
				*Ar << Filenames;
				*Ar << ChunkIds;
			}

			check(Filenames.Num() == ChunkIds.Num());

			for (int32 Idx = 0, Num = Filenames.Num(); Idx < Num; ++Idx)
			{
				UE_LOGF(LogCookOnTheFly, Verbose, "Adding file '%ls'", *Filenames[Idx]);
				ServerToc.AddFile(ChunkIds[Idx], Filenames[Idx], STORAGE_SERVER_FILE_UNKOWN_SIZE);
			}

			break;
		}
	}
}
#endif

#endif
