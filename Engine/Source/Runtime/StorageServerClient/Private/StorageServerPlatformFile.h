// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Containers/StringView.h"
#include "Containers/DirectoryTree.h"
#include "IO/IoDispatcher.h"
#include "IStorageServerPlatformFile.h"
#include "StorageServerConnection.h"
#include "StorageServerIoDispatcherBackend.h"
#include "StorageServerPackageStore.h"

#if !UE_BUILD_SHIPPING

class FStorageServerFileHandle;
class FStorageServerConnection;
class IPackageStore;

#if WITH_COTF
namespace UE::Cook
{
	class FCookOnTheFlyMessage;
	class ICookOnTheFlyServerConnection;
}
#endif

#define STORAGE_SERVER_FILE_UNKOWN_SIZE (-1) 

class FStorageServerFileSystemTOC
{
public:
	~FStorageServerFileSystemTOC();
	void AddFile(const FIoChunkId& FileChunkId, FStringView Path, int64 RawSize);
	void Clear();
	bool FileExists(const FString& Path);
	bool DirectoryExists(const FString& Path);
	const FIoChunkId* GetFileChunkId(const FString& Path);
	int64 GetFileSize(const FString& Path);
	bool GetFileData(const FString& Path, FIoChunkId& OutChunkId, int64& OutRawSize);
	bool IterateDirectory(const FString& Path, TFunctionRef<bool(const FIoChunkId&, const TCHAR*, int64)> Callback);
	bool IterateDirectoryRecursively(const FString& Path, TFunctionRef<bool(const FIoChunkId&, const TCHAR*, int64)> Callback);

private:
	struct FDirectory
	{
		TArray<FString> Directories;
		TArray<int32> Files;
	};

	struct FFile
	{
		FIoChunkId FileChunkId;
		FString FilePath;
		int64 RawSize;
	};

	FDirectory* AddDirectoriesRecursive(const FString& DirectoryPath);

	FDirectory Root;
	TMap<FString, FDirectory*> Directories;
	TMap<FString, int32> FilePathToIndexMap;
	TArray<FFile> Files;
	FRWLock TocLock;
};

class FStorageServerPlatformFile
	: public IStorageServerPlatformFile
{
public:
	FStorageServerPlatformFile();
	virtual ~FStorageServerPlatformFile();
	virtual bool ShouldBeUsed(IPlatformFile* Inner, const TCHAR* CmdLine) const override;
	virtual bool Initialize(IPlatformFile* Inner, const TCHAR* CmdLine) override;
	virtual void InitializeAfterProjectFilePath() override;

	virtual IPlatformFile* GetLowerLevel() override
	{
		return LowerLevel;
	}

	virtual void SetLowerLevel(IPlatformFile* NewLowerLevel) override
	{
		LowerLevel = NewLowerLevel;
	}

	virtual const TCHAR* GetName() const override
	{
		return TEXT("StorageServer");
	}

	virtual bool FileExists(const TCHAR* Filename) override;
	virtual int64 FileSize(const TCHAR* Filename) override;
	virtual bool IsReadOnly(const TCHAR* Filename) override;
	virtual FDateTime GetTimeStamp(const TCHAR* Filename) override;
	virtual FDateTime GetAccessTimeStamp(const TCHAR* Filename) override;
	virtual IFileHandle* OpenRead(const TCHAR* Filename, bool bAllowWrite = false) override;
	virtual bool DirectoryExists(const TCHAR* Directory) override;
	virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) override;
	virtual bool IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor) override;
	virtual bool IterateDirectoryRecursively(const TCHAR* Directory, FDirectoryVisitor& Visitor) override;
	virtual bool IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor) override;
	virtual FOpenMappedResult OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions = EOpenReadFlags::None, int64 MaximumSize = 0) override;
	virtual FOpenMappedResult OpenMappedEx2(const TCHAR* Filename, EOpenMappedFlags OpenOptions = EOpenMappedFlags::None, int64 MaximumSize = 0) override;
	virtual FString GetFilenameOnDisk(const TCHAR* Filename) override;
	virtual bool DeleteFile(const TCHAR* Filename) override;
	virtual bool MoveFile(const TCHAR* To, const TCHAR* From) override;
	virtual bool SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) override;
	virtual void SetTimeStamp(const TCHAR* Filename, FDateTime DateTime) override;
	virtual IFileHandle* OpenWrite(const TCHAR* Filename, bool bAppend = false, bool bAllowRead = false) override;
	virtual bool CreateDirectory(const TCHAR* Directory) override;
	virtual bool DeleteDirectory(const TCHAR* Directory) override;
	virtual FString ConvertToAbsolutePathForExternalAppForRead(const TCHAR* Filename) override;
	virtual FString ConvertToAbsolutePathForExternalAppForWrite(const TCHAR* Filename) override;
	virtual bool SendMessageToServer(const TCHAR* Message, IPlatformFile::IFileServerMessageHandler* Handler) override;

	FStringView GetHostAddr() const override;
	void GetAndResetConnectionStats(FConnectionStats& OutStats) override;

	void SetAllowPackageIo(bool bInAllowPackageIo)
	{
		bAllowPackageIo = bInAllowPackageIo;
	}
	void SetAbortOnConnectionFailure(bool bInAbortOnConnectionFailure)
	{
		bAbortOnConnectionFailure = bInAbortOnConnectionFailure;
	}
	void SetCustomProjectStorePath(FStringView InProjectStorePath)
	{
		CustomProjectStorePath = InProjectStorePath;
	}
	
	void UpdateFileList() override;

	bool IsConnectionValid() const
	{
		return Connection.IsValid();
	}

	TSharedPtr<IPackageStoreBackend> CreatePackageStoreBackend() const;
	TSharedPtr<IIoDispatcherBackend> CreateIoDispatcherBackend() const;

private:
	friend class FStorageServerFileHandle;
	template<typename ParentVisitorClass, typename DataType> friend class FUniqueDirectoryStatVisitor;


	void InitializeConnection();
	bool IsNonServerFilenameAllowed(FStringView InFilename);
	bool IsAssumedImmutableTimeStampFilename(FStringView InFilename) const;
	bool IsEngineStartupPrecachableFilename(FStringView InFilename) const;
	bool MakeStorageServerPath(const TCHAR* LocalFilenameOrDirectory, FStringBuilderBase& OutPath) const;
	bool MakeLocalPath(const TCHAR* ServerFilenameOrDirectory, FStringBuilderBase& OutPath) const;
	IFileHandle* InternalOpenFile(const FIoChunkId& FileChunkId, int64 RawSize, const TCHAR* LocalFilename);
	bool SendGetFileListMessage();
	FFileStatData SendGetStatDataMessage(const FIoChunkId& FileChunkId);
	int64 SendReadMessage(uint8* Destination, const FIoChunkId& FileChunkId, int64 Offset, int64 BytesToRead);
#if WITH_COTF
	void OnCookOnTheFlyMessage(const UE::Cook::FCookOnTheFlyMessage& Message);
#endif
	TUniquePtr<FArchive> TryFindProjectStoreMarkerFile(IPlatformFile* Inner) const;
	FAnsiString MakeBaseURI();

	FString CustomProjectStorePath;
	TSet<FName> ExcludedNonServerExtensions;
	TSet<FName> AssumedImmutableTimeStampExtensions;
	TSet<FName> EngineStartupPrecacheExtensions;
	IPlatformFile* LowerLevel = nullptr;
	FStringView ServerEngineDirView = FStringView(TEXT("/{engine}/"));
	FStringView ServerProjectDirView = FStringView(TEXT("/{project}/"));
	TUniquePtr<FStorageServerConnection> Connection;
#if WITH_COTF
	TSharedPtr<UE::Cook::ICookOnTheFlyServerConnection> CookOnTheFlyServerConnection;
#endif
	FStorageServerFileSystemTOC ServerToc;
	FString ServerProject;
	FString ServerPlatform;
	FString BaseURI;
	FString AbsProjectDir;
	FString AbsEngineDir;
	FString WorkspaceSharePath;
	TDirectoryTree<FString> RemapDirectoriesTree;
	mutable TArray<FString> HostAddrs;
	mutable uint16 HostPort = 8558;
	TArray<FAnsiString> AuthHeaders;
	bool bAllowPackageIo = true;
	bool bAbortOnConnectionFailure = true;
};

#endif
