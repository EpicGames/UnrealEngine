// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "HAL/IConsoleManager.h"

#include "ZenStreamingSettings.generated.h"

UCLASS(MinimalAPI, config=Game, defaultconfig, meta=(DisplayName="Zen Streaming"))
class UZenStreamingSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	/**
	* Show an in-game badge to indicate if the client is streaming from Zen
	*/
	UPROPERTY(config, EditAnywhere, Category = ZenStreaming, meta=(DisplayName="Show Zen streaming badge", ConsoleVariable="zen.indicator.show"))
	bool bShowZenBadge = true;

	UPROPERTY(config, EditAnywhere, Category = ZenStreaming, meta=(ClampMin=0, ClampMax=1.0, ConsoleVariable="zen.indicator.x"))
	float BadgePositionX = 0.05f;

	UPROPERTY(config, EditAnywhere, Category = ZenStreaming, meta=(ClampMin=0, ClampMax=1.0, ConsoleVariable="zen.indicator.y"))
	float BadgePositionY = 0.8f;

	/**
	 * For how long (in seconds) should the badge stay on screen. < 0 will make the badge persistent
	 */
	UPROPERTY(config, EditAnywhere, Category = ZenStreaming, meta=(ClampMin=-1, ConsoleVariable="zen.indicator.fadetime"))
	float BadgeFadeTime = -1;

	UPROPERTY(config, EditAnywhere, Category = ZenStreaming, meta=(ClampMin=0, ClampMax=1.0, ConsoleVariable="zen.indicator.alpha"))
	float BadgeAlpha = 0.5;
};
